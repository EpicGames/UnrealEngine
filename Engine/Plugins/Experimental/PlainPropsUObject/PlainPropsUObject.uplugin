{
	"FileVersion": 3,
	"FriendlyName": "PlainPropsUObject",
	"Description": "New Serialization Stack Prototype - CoreUObject Bindings",
	"Category": "Experimental",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": false,
	"IsExperimentalVersion": true,
	"SupportedTargetPlatforms": [
		"Win64"
	],
	"Modules": [
		{
			"Name": "PlainPropsUObject",
			"Type": "Runtime",
			"LoadingPhase": "PreLoadingScreen",
			"PlatformAllowList": [ "Win64" ]
		}
	],
	"Plugins": [
		{
			"Name": "PlainProps",
			"Enabled": true
		}
	]
}
