// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsUObjectRuntime.h"
#include "UObject/Class.h"

////////////////////////////////////////////////////////////////////////////////////////////////
// Temporary private UE_CUSTOM_BIND_MEMBERS helpers

#define _PP_MEMBER_ID(N, NS, T, M) Ids::IndexMember(#M),

#define _PP_CUSTOM_BIND_MEMBERS(N, ...) \
	struct M { enum : uint16 { __VA_ARGS__ }; }; \
	inline static constexpr int NumMembers = N; \
	const TStaticArray<FMemberId, NumMembers> MemberIds = { _PP_EXPAND(_PP_REFLECT_THINGS(N, _PP_MEMBER_ID, _, _, __VA_ARGS__)) };

// Temporary UE_CUSTOM_BIND_MEMBERS macro that eliminates the manual hassle of 
// of sizing arrays, indexing names and keeping track of indices when modifying custom bindings.
// Todo: Handle Name-Spec pairs and interleaving of custom members with schema members.
#define UE_CUSTOM_BIND_MEMBERS(T, ...) \
	using Type = T; \
	using Ids = FRuntimeIds; \
	_PP_CUSTOM_BIND_MEMBERS(_PP_NUM_ARGS(__VA_ARGS__), __VA_ARGS__) \

////////////////////////////////////////////////////////////////////////////////////////////////

// Temporary macro to switch between property override loading and object override loading
#define PP_OVERRIDE_OBJECT_LOADING 0

PP_REFLECT_STRUCT(, FImplementedInterface, void, Class, PointerOffset, bImplementedByK2);

PP_REFLECT_FLAG_ENUM(, EClassFlags, 
	CLASS_None,
	CLASS_Abstract,
	CLASS_DefaultConfig,
	CLASS_Config,
	CLASS_Transient,
	CLASS_Optional,
	CLASS_MatchedSerializers,
	CLASS_ProjectUserConfig,
	CLASS_Native,
	CLASS_NotPlaceable,
	CLASS_PerObjectConfig,
	CLASS_ReplicationDataIsSetUp,
	CLASS_EditInlineNew,
	CLASS_CollapseCategories,
	CLASS_Interface,
	CLASS_PerPlatformConfig,
	CLASS_Const,
	CLASS_NeedsDeferredDependencyLoading,
	CLASS_CompiledFromBlueprint,
	CLASS_MinimalAPI,
	CLASS_RequiredAPI,
	CLASS_DefaultToInstanced,
	CLASS_TokenStreamAssembled,
	CLASS_HasInstancedReference,
	CLASS_Hidden,
	CLASS_Deprecated,
	CLASS_HideDropDown,
	CLASS_GlobalUserConfig,
	CLASS_Intrinsic,
	CLASS_Constructed,
	CLASS_ConfigDoNotCheckDefaults,
	CLASS_NewerVersionExists
);

PP_REFLECT_FLAG_ENUM(, EFunctionFlags, 
	FUNC_None,
	FUNC_Final,
	FUNC_RequiredAPI,
	FUNC_BlueprintAuthorityOnly,
	FUNC_BlueprintCosmetic,
	FUNC_Net,
	FUNC_NetReliable,
	FUNC_NetRequest,
	FUNC_Exec,
	FUNC_Native,
	FUNC_Event,
	FUNC_NetResponse,
	FUNC_Static,
	FUNC_NetMulticast,
	FUNC_UbergraphFunction,
	FUNC_MulticastDelegate,
	FUNC_Public,
	FUNC_Private,
	FUNC_Protected,
	FUNC_Delegate,
	FUNC_NetServer,
	FUNC_HasOutParms,
	FUNC_HasDefaults,
	FUNC_NetClient,
	FUNC_DLLImport,
	FUNC_BlueprintCallable,
	FUNC_BlueprintEvent,
	FUNC_BlueprintPure,
	FUNC_EditorOnly,
	FUNC_Const,
	FUNC_NetValidate
);

PP_REFLECT_FLAG_ENUM(, EStructFlags, 
	STRUCT_NoFlags,
	STRUCT_Native,
	STRUCT_IdenticalNative,
	STRUCT_HasInstancedReference,
	STRUCT_NoExport,
	STRUCT_Atomic,
	STRUCT_Immutable,
	STRUCT_AddStructReferencedObjects,
	STRUCT_RequiredAPI,
	STRUCT_NetSerializeNative,
	STRUCT_SerializeNative,
	STRUCT_CopyNative,
	STRUCT_IsPlainOldData,
	STRUCT_NoDestructor,
	STRUCT_ZeroConstructor,
	STRUCT_ExportTextItemNative,
	STRUCT_ImportTextItemNative,
	STRUCT_PostSerializeNative,
	STRUCT_SerializeFromMismatchedTag,
	STRUCT_NetDeltaSerializeNative,
	STRUCT_PostScriptConstruct,
	STRUCT_NetSharedSerialization,
	STRUCT_Trashed,
	STRUCT_NewerVersionExists,
	STRUCT_CanEditChange,
	STRUCT_Visitor,
	STRUCT_PostLoad
);

PP_REFLECT_FLAG_ENUM(, EPropertyFlags, 
	CPF_None,
	CPF_Edit,
	CPF_ConstParm,
	CPF_BlueprintVisible,
	CPF_ExportObject,
	CPF_BlueprintReadOnly,
	CPF_Net,
	CPF_EditFixedSize,
	CPF_Parm,
	CPF_OutParm,
	CPF_ZeroConstructor,
	CPF_ReturnParm,
	CPF_DisableEditOnTemplate,
	CPF_NonNullable,
	CPF_Transient,
	CPF_Config,
	CPF_RequiredParm,
	CPF_DisableEditOnInstance,
	CPF_EditConst,
	CPF_GlobalConfig,
	CPF_InstancedReference,
	CPF_ExperimentalExternalObjects,
	CPF_DuplicateTransient,
	CPF_SaveGame,
	CPF_NoClear,
	CPF_Virtual,
	CPF_ReferenceParm,
	CPF_BlueprintAssignable,
	CPF_Deprecated,
	CPF_IsPlainOldData,
	CPF_RepSkip,
	CPF_RepNotify,
	CPF_Interp,
	CPF_NonTransactional,
	CPF_EditorOnly,
	CPF_NoDestructor,
	CPF_AutoWeak,
	CPF_ContainsInstancedReference,
	CPF_AssetRegistrySearchable,
	CPF_SimpleDisplay,
	CPF_AdvancedDisplay,
	CPF_Protected,
	CPF_BlueprintCallable,
	CPF_BlueprintAuthorityOnly,
	CPF_TextExportTransient,
	CPF_NonPIEDuplicateTransient,
	CPF_ExposeOnSpawn,
	CPF_PersistentInstance,
	CPF_UObjectWrapper,
	CPF_HasGetValueTypeHash,
	CPF_NativeAccessSpecifierPublic,
	CPF_NativeAccessSpecifierProtected,
	CPF_NativeAccessSpecifierPrivate,
	CPF_SkipSerialization,
	CPF_TObjectPtr,
	CPF_ExperimentalOverridableLogic,
	CPF_ExperimentalAlwaysOverriden,
	CPF_ExperimentalNeverOverriden,
	CPF_AllowSelfReference,
	CPF_ForcePostConstructLink
);

namespace PlainProps::UE
{

// Todo: Move back to PlainPropsUObjectBindingsInternal.h,
// where it is used to bind dynamic structs as they are being loaded.
// It is temporarily exposed here for allowing duplication of a UStruct binding with a different bind id.
PLAINPROPSUOBJECT_API void BindStruct(FBindId Id, const UStruct* Struct);

////////////////////////////////////////////////////////////////////////////////////////////////
// Native FProperty meta bindings

struct FPropertyBinding : ICustomBinding
{
	UE_CUSTOM_BIND_MEMBERS(FProperty,
		NamePrivate, ArrayDim, ElementSize, PropertyFlags, RepIndex, BlueprintReplicationCondition);
	FPropertyBinding(TPropertySpecifier<NumMembers>& Spec);

	const FEnumId PropertyFlagsId;
	const FEnumId LifetimeConditionId;

	void		Save(FMemberBuilder& Dst, const Type& Src, const Type* Default, const FSaveContext& Ctx) const;
	void		Load(Type& Dst, FStructLoadView Src, ECustomLoadMethod Method) const;
	static bool	Diff(const Type& A, const Type& B, const FBindContext& Ctx) { return &A != &B; }
};

struct FBoolPropertyBinding : ICustomBinding
{
	UE_CUSTOM_BIND_MEMBERS(FBoolProperty, FieldSize, ByteOffset, ByteMask, FieldMask);
	FBoolPropertyBinding(TPropertySpecifier<NumMembers>& Spec);

	void		Save(FMemberBuilder& Dst, const Type& Src, const Type* Default, const FSaveContext& Ctx) const;
	void		Load(Type& Dst, FStructLoadView Src, ECustomLoadMethod Method) const;
	static bool	Diff(const Type& A, const Type& B, const FBindContext& Ctx) { return &A != &B; }
};

struct FStructPropertyBinding : ICustomBinding
{
	UE_CUSTOM_BIND_MEMBERS(FStructProperty, Struct);
	FStructPropertyBinding(TPropertySpecifier<NumMembers>& Spec);

	void		Save(FMemberBuilder& Dst, const Type& Src, const Type* Default, const FSaveContext& Ctx) const;
	void		Load(Type& Dst, FStructLoadView Src, ECustomLoadMethod Method) const;
	static bool	Diff(const Type& A, const Type& B, const FBindContext& Ctx) { return &A != &B; }
};

struct FClassPropertyBinding : ICustomBinding
{
	UE_CUSTOM_BIND_MEMBERS(FClassProperty, MetaClass);
	FClassPropertyBinding(TPropertySpecifier<NumMembers>& Spec);

	void		Save(FMemberBuilder& Dst, const Type& Src, const Type* Default, const FSaveContext& Ctx) const;
	void		Load(Type& Dst, FStructLoadView Src, ECustomLoadMethod Method) const;
	static bool	Diff(const Type& A, const Type& B, const FBindContext& Ctx) { return &A != &B; }
};

struct FObjectPropertyBaseBinding : ICustomBinding
{
	UE_CUSTOM_BIND_MEMBERS(FObjectPropertyBase, PropertyClass);
	FObjectPropertyBaseBinding(TPropertySpecifier<NumMembers>& Spec);

	void		Save(FMemberBuilder& Dst, const Type& Src, const Type* Default, const FSaveContext& Ctx) const;
	void		Load(Type& Dst, FStructLoadView Src, ECustomLoadMethod Method) const;
	static bool	Diff(const Type& A, const Type& B, const FBindContext& Ctx) { return &A != &B; }
};

struct FArrayPropertyBinding : ICustomBinding
{
	UE_CUSTOM_BIND_MEMBERS(FArrayProperty, InnerPropertyTypeName, InnerProperty);
	FArrayPropertyBinding(TPropertySpecifier<NumMembers>& Spec);

	void		Save(FMemberBuilder& Dst, const Type& Src, const Type* Default, const FSaveContext& Ctx) const;
	void		Load(Type& Dst, FStructLoadView Src, ECustomLoadMethod Method) const;
	static bool	Diff(const Type& A, const Type& B, const FBindContext& Ctx) { return &A != &B; }
};

struct FSetPropertyBinding : ICustomBinding
{
	UE_CUSTOM_BIND_MEMBERS(FSetProperty, InnerPropertyTypeName, InnerProperty);
	FSetPropertyBinding(TPropertySpecifier<NumMembers>& Spec);

	void		Save(FMemberBuilder& Dst, const Type& Src, const Type* Default, const FSaveContext& Ctx) const;
	void		Load(Type& Dst, FStructLoadView Src, ECustomLoadMethod Method) const;
	static bool	Diff(const Type& A, const Type& B, const FBindContext& Ctx) { return &A != &B; }
};

struct FMapPropertyBinding : ICustomBinding
{
	UE_CUSTOM_BIND_MEMBERS(FMapProperty, KeyPropertyTypeName, KeyProperty, ValuePropertyTypeName, ValueProperty);
	FMapPropertyBinding(TPropertySpecifier<NumMembers>& Spec);

	void		Save(FMemberBuilder& Dst, const Type& Src, const Type* Default, const FSaveContext& Ctx) const;
	void		Load(Type& Dst, FStructLoadView Src, ECustomLoadMethod Method) const;
	static bool	Diff(const Type& A, const Type& B, const FBindContext& Ctx) { return &A != &B; }
};

struct FOptionalPropertyBinding : ICustomBinding
{
	UE_CUSTOM_BIND_MEMBERS(FOptionalProperty, ValuePropertyTypeName, ValueProperty);
	FOptionalPropertyBinding(TPropertySpecifier<NumMembers>& Spec);

	void		Save(FMemberBuilder& Dst, const Type& Src, const Type* Default, const FSaveContext& Ctx) const;
	void		Load(Type& Dst, FStructLoadView Src, ECustomLoadMethod Method) const;
	static bool	Diff(const Type& A, const Type& B, const FBindContext& Ctx) { return &A != &B; }
};

struct FBytePropertyBinding : ICustomBinding
{
	UE_CUSTOM_BIND_MEMBERS(FByteProperty, Enum);
	FBytePropertyBinding(TPropertySpecifier<NumMembers>& Spec);

	void		Save(FMemberBuilder& Dst, const Type& Src, const Type* Default, const FSaveContext& Ctx) const;
	void		Load(Type& Dst, FStructLoadView Src, ECustomLoadMethod Method) const;
	static bool	Diff(const Type& A, const Type& B, const FBindContext& Ctx) { return &A != &B; }
};

struct FEnumPropertyBinding : ICustomBinding
{
	UE_CUSTOM_BIND_MEMBERS(FEnumProperty, UnderlyingPropertyTypeName, UnderlyingProperty, Enum);
	FEnumPropertyBinding(TPropertySpecifier<NumMembers>& Spec);

	void		Save(FMemberBuilder& Dst, const Type& Src, const Type* Default, const FSaveContext& Ctx) const;
	void		Load(Type& Dst, FStructLoadView Src, ECustomLoadMethod Method) const;
	static bool	Diff(const Type& A, const Type& B, const FBindContext& Ctx) { return &A != &B; }
};

////////////////////////////////////////////////////////////////////////////////////////////////
// Native UField meta bindings

struct UStructBinding : ICustomBinding
{
	UE_CUSTOM_BIND_MEMBERS(UStruct, SuperStruct, Children, ChildPropertyClasses);
	UStructBinding(TPropertySpecifier<NumMembers>& Spec);

	static FTypedRange SaveChildrenStatic(const UField* Children, int32 Num, const FSaveContext& Ctx);
	static FTypedRange SaveChildPropertyClassesStatic(const FField* ChildProperties, int32 Num, const FSaveContext& Ctx);
	static void LoadChildrenStatic(TObjectPtr<UField>& Children, FStructRangeLoadView Src);
	static void LoadChildPropertyClassesStatic(UStruct& Dst, FStructRangeLoadView Src);

	void		Save(FMemberBuilder& Dst, const Type& Src, const Type* Default, const FSaveContext& Ctx) const;
	void		Load(Type& Dst, FStructLoadView Src, ECustomLoadMethod Method) const;
	static bool	Diff(const Type& A, const Type& B, const FBindContext& Ctx) { return &A != &B; }
};

struct UClassBinding : ICustomBinding
{
#if WITH_EDITORONLY_DATA
	UE_CUSTOM_BIND_MEMBERS(UClass, ClassFlags, ClassWithin, ClassConfigName, ClassGeneratedBy, Interfaces);
#else
	UE_CUSTOM_BIND_MEMBERS(UClass, ClassFlags, ClassWithin, ClassConfigName, Interfaces);
#endif
	UClassBinding(TPropertySpecifier<NumMembers>& Spec);

	const FEnumId ClassFlagsId;
	const FDualStructId ImplementedInterfaceId;

	static FTypedRange SaveInterfacesStatic(FBindId Id, TConstArrayView<FImplementedInterface> Interfaces, const FSaveContext& Ctx);
	static void LoadInterfacesStatic(TArray<FImplementedInterface>& Interfaces, FStructRangeLoadView Src);

	void		Save(FMemberBuilder& Dst, const Type& Src, const Type* Default, const FSaveContext& Ctx) const;
	void		Load(Type& Dst, FStructLoadView Src, ECustomLoadMethod Method) const;
	static bool	Diff(const Type& A, const Type& B, const FBindContext& Ctx) { return &A != &B; }
};

struct UFunctionBinding : ICustomBinding
{
	UE_CUSTOM_BIND_MEMBERS(UFunction, FunctionFlags, EventGraphFunction, EventGraphCallOffset);
	UFunctionBinding(TPropertySpecifier<NumMembers>& Spec);

	const FEnumId FunctionFlagsId;

	void		Save(FMemberBuilder& Dst, const Type& Src, const Type* Default, const FSaveContext& Ctx) const;
	void		Load(Type& Dst, FStructLoadView Src, ECustomLoadMethod Method) const;
	static bool	Diff(const Type& A, const Type& B, const FBindContext& Ctx) { return &A != &B; }
};

struct UScriptStructBinding : ICustomBinding
{
	UE_CUSTOM_BIND_MEMBERS(UScriptStruct, StructFlags);
	UScriptStructBinding(TPropertySpecifier<NumMembers>& Spec);

	const FEnumId StructFlagsId;

	void		Save(FMemberBuilder& Dst, const Type& Src, const Type* Default, const FSaveContext& Ctx) const;
	void		Load(Type& Dst, FStructLoadView Src, ECustomLoadMethod Method) const;
	static bool	Diff(const Type& A, const Type& B, const FBindContext& Ctx) { return &A != &B; }
};

////////////////////////////////////////////////////////////////////////////////////////////////

} // namespace PlainProps::UE

namespace PlainProps
{
// custom FProperty meta bindings
template<> struct TCustomBind<FProperty> { using Type = UE::FPropertyBinding; };
template<> struct TCustomBind<FEnumProperty> { using Type = UE::FEnumPropertyBinding; };
template<> struct TCustomBind<FBoolProperty> { using Type = UE::FBoolPropertyBinding; };
template<> struct TCustomBind<FByteProperty> { using Type = UE::FBytePropertyBinding; };
template<> struct TCustomBind<FStructProperty> { using Type = UE::FStructPropertyBinding; };
template<> struct TCustomBind<FClassProperty> { using Type = UE::FClassPropertyBinding; };
template<> struct TCustomBind<FObjectPropertyBase> { using Type = UE::FObjectPropertyBaseBinding; };
template<> struct TCustomBind<FArrayProperty> { using Type = UE::FArrayPropertyBinding; };
template<> struct TCustomBind<FSetProperty> { using Type = UE::FSetPropertyBinding; };
template<> struct TCustomBind<FMapProperty> { using Type = UE::FMapPropertyBinding; };
template<> struct TCustomBind<FOptionalProperty> { using Type = UE::FOptionalPropertyBinding; };

// custom UField meta bindings
template<> struct TCustomBind<UStruct> { using Type = UE::UStructBinding; };
template<> struct TCustomBind<UClass> { using Type = UE::UClassBinding; };
template<> struct TCustomBind<UFunction> { using Type = UE::UFunctionBinding; };
template<> struct TCustomBind<UScriptStruct> { using Type = UE::UScriptStructBinding; };

// traits for aux bindings
template<> struct TOccupancyOf<FImplementedInterface> : FRequireAll {};
template<> inline FScopeId GetOuterScope<UE::FRuntimeIds, TTypename<FImplementedInterface>>() { return UE::GUE.Scopes.CoreUObject; }
}
