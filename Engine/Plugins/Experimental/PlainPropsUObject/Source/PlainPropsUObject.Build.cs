// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PlainPropsUObject : ModuleRules
{
	public PlainPropsUObject(ReadOnlyTargetRules Target) : base(Target)
	{
		bDisableStaticAnalysis = true;
		bValidateInternalApi = false; // for UE_INTERNAL FLinkerLoad::ResolveResource access

		CppStandard = CppStandardVersion.Cpp20;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"CorePreciseFP",
				"JsonObjectGraph",	// for UE::JsonObjectGraph::Stringify for debugging
				"PlainProps"
		});
	}
}
