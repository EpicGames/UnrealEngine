<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">
  
  <!-- // Copyright Epic Games, Inc. All Rights Reserved. -->

  <!-- PlainPropsRoundtripTest.cpp -->

  <Type Name="PlainProps::UE::TCustomPropertyBinding&lt;*,*&gt;">
    <DisplayString>{"$T1",sb} in {"$T2",sb}</DisplayString>
  </Type>

</AutoVisualizer>
