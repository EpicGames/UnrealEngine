// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsRestoreOverridableInternal.h"
#include "PlainPropsInternalPrivateMemberPtr.h"
#include "PlainPropsBind.h"
#include "PlainPropsUObjectRuntime.h"
#include "Containers/Array.h"
#include "Serialization/ArchiveSerializedPropertyChain.h"
#include "UObject/Class.h"
#include "UObject/OverriddenPropertySet.h"
#include "UObject/UnrealType.h"

namespace PlainProps::UE
{

// Todo: This should be moved to an appropriate context that we can pass around.
thread_local FOverriddenPropertySet* TlsRestoreOverrides = nullptr;

PP_DEFINE_PRIVATE_MEMBER_NAMESPACE_BEGIN(FOverriddenPropertyNode_Private)
PP_DEFINE_PRIVATE_MEMBER_PTR(FOverriddenPropertyNode, Operation, EOverriddenPropertyOperation);
PP_DEFINE_PRIVATE_MEMBER_PTR(FOverriddenPropertyNode, SubPropertyNodes, TArray<FOverriddenPropertyNode>);
PP_DEFINE_PRIVATE_MEMBER_NAMESPACE_END()

void ResetOverriddenPropertyNode(FOverriddenPropertyNode& Node, EOverriddenPropertyOperation Op, TArray<FOverriddenPropertyNode> SubPropertyNodes)
{
	using namespace FOverriddenPropertyNode_Private;
	Node.*_Operation = Op;
	Node.*_SubPropertyNodes = MoveTemp(SubPropertyNodes);
}

// Determines whether an override operation is Modified or Replaced. A struct member is considered 
// replaced if all of its members have been serialized. 
// Note: in FOverriddenPropertySet::NotifyPropertyChange we consider an edited property as
// 'Replaced' when StructProperty->Struct->UseNativeSerialization() is true.
static EOverriddenPropertyOperation DetermineOverrideOperation(FStructView SerializedStruct, 
															   const FRestoreContext& Ctx)
{
	FMemberReader It(SerializedStruct);

	// Handle super struct separately
	if (It.HasSuper())
	{
		EOverriddenPropertyOperation SubOp = DetermineOverrideOperation(It.GrabStruct(), Ctx);
		if (SubOp == EOverriddenPropertyOperation::Modified)
		{
			// When the sub-operation is Modified, then this member is not fully
			// replaced and more sub-operations will be nested under it.
			return EOverriddenPropertyOperation::Modified;
		}
	}
	uint16 NumSerializedMembers = 0;
	while (It.HasMore())
	{
		if (It.PeekKind() == EMemberKind::Struct)
		{
			EOverriddenPropertyOperation SubOp = DetermineOverrideOperation(It.GrabStruct(), Ctx);
			if (SubOp == EOverriddenPropertyOperation::Modified)
			{
				// When the sub-operation is Modified, then this member is not fully
				// replaced and more sub-operations will be nested under it.
				return EOverriddenPropertyOperation::Modified;
			}
		}
		else
		{
			It.SkipMember();
		}
		++NumSerializedMembers;
	}

	// This is a temporary workaround to find whether we have serialized all members or not.
	// This could be optimized but we could also be using LoadIds instead.
	// Note: assume we are using ESchemaFormat::StableNames.
	check(SerializedStruct.Schema.Id.Idx < static_cast<uint32>(Ctx.RuntimeIds.Num()));
	FBindId BindId = FBindId(reinterpret_cast<const FDeclId*>(Ctx.RuntimeIds.GetData())[SerializedStruct.Schema.Id.Idx]);

	uint16 NumDeclMembers = 0;
	const FStructDeclaration* Decl = nullptr;
	if (Ctx.Schemas.FindStruct(BindId, Decl))
	{
		NumDeclMembers = Decl->NumMembers;
	}
	else if (const FStructDeclaration* CustomDecl = Ctx.Customs.FindDeclaration(BindId))
	{
		NumDeclMembers = CustomDecl->NumMembers;
	}
	// if there's no declaration this is a primitive type bound as a struct i.e object pointer, consider it replaced
	else
	{
		ensure(NumSerializedMembers >= NumDeclMembers);
		NumDeclMembers = NumSerializedMembers;
	}

	return (NumSerializedMembers == NumDeclMembers) ? EOverriddenPropertyOperation::Replace : EOverriddenPropertyOperation::Modified;
}

static void RestoreStruct(const UStruct* Struct, FStructView StructView, FOverriddenPropertySet& Overrides, FArchiveSerializedPropertyChain& PropertyChain, const FRestoreContext& Ctx);

static void RestoreProperty(FProperty* Property, 
							FMemberType Type, 
							FArchiveSerializedPropertyChain& PropertyChain,
							FOverriddenPropertySet& Overrides,
							FMemberReader& Reader,
							const FRestoreContext& Ctx)
{
	FStructView SubStructView;

	// Properties with the ExperimentalOverridableLogic flag have already been restored
	// during loading via a custom binding.
	bool bSkipOverride = Property->HasAnyPropertyFlags(CPF_ExperimentalOverridableLogic);

	EOverriddenPropertyOperation Op = EOverriddenPropertyOperation::Replace;

	switch (Type.GetKind())
	{
		case EMemberKind::Leaf:
			Reader.GrabLeaf();
			break;

		case EMemberKind::Struct:
		{
			SubStructView = Reader.GrabStruct();

			if (!bSkipOverride)
			{
				Op = DetermineOverrideOperation(SubStructView, Ctx);
			}
		}
		break;

		case EMemberKind::Range:
			Reader.GrabRange();
			break;
	}

	if (!bSkipOverride)
	{
		check(Op != EOverriddenPropertyOperation::None);

		Overrides.RestoreOverriddenPropertyOperation(Op, &PropertyChain, Property);

		if (Type.GetKind() == EMemberKind::Struct)
		{
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				PropertyChain.PushProperty(Property, Property->IsEditorOnlyProperty());
				RestoreStruct(StructProperty->Struct, SubStructView, Overrides, PropertyChain, Ctx);
				PropertyChain.PopProperty(Property, Property->IsEditorOnlyProperty());
			}
		}		
	}
}

static void RestoreStruct(const UStruct* Struct, 
							FStructView StructView,
							FOverriddenPropertySet& Overrides,
							FArchiveSerializedPropertyChain& PropertyChain,
							const FRestoreContext& Ctx)
{	
	FMemberReader It(StructView);

	while (It.HasMore())
	{
		if (It.HasSuper())
		{
			FStructView Super = It.GrabStruct();
			RestoreStruct(Struct, Super, Overrides, PropertyChain, Ctx);
			continue;
		}
		if (FOptionalMemberId Name = It.PeekName())
		{
			FMemberType Type = It.PeekType();

			FName MemberFName = GUE.Names.ResolveName(Name.Get().Id).ToName();
			checkSlow(MemberFName != NAME_None);

			if (FProperty* Property = Struct->FindPropertyByName(MemberFName))
			{
				RestoreProperty(Property, Type, PropertyChain,
								Overrides, It, Ctx);

				continue;
			}
		}
		It.SkipMember();
	}
}

void FlattenModifiedOverrides(FOverriddenPropertyNode& Root, FStructSchemaId Id, const FRestoreContext& Ctx)
{
	if (Root.GetOperation() == EOverriddenPropertyOperation::Modified)
	{
		FBindId BindId = FBindId(reinterpret_cast<const FDeclId*>(Ctx.RuntimeIds.GetData())[Id.Idx]);
		uint32 NumDeclMembers = 0;
		uint32 OverriddenCount = 0;
		
		FOptionalDeclId DeclId = FOptionalDeclId(LowerCast(BindId));
		while (DeclId)
		{
			const FStructDeclaration* Decl = nullptr;
			BindId = UpCast(DeclId.Get());
			if (!Ctx.Schemas.FindStruct(BindId, Decl))
			{
				Decl = Ctx.Customs.FindDeclaration(BindId);
			}
			checkf(Decl, TEXT("'%s' declaration not found"), *Ctx.Schemas.GetDebug().Print(BindId));
			for (EPropertyFlags MemberFlag : Ctx.Metadatas.GetMemberFlags(BindId))
			{
				OverriddenCount += !!(MemberFlag & EPropertyFlags::CPF_ExperimentalAlwaysOverriden);
			}
			if (Decl)
			{
				NumDeclMembers += Decl->NumMembers;
				DeclId = Decl->Super;
			}
			else
			{
				DeclId = NoId;
			}
		}
		
		for (const FOverriddenPropertyNode& SubNodes : Root.GetSubPropertyNodes())
		{
			if (SubNodes.GetOperation() == EOverriddenPropertyOperation::Replace)
			{
				++OverriddenCount;
			}
		}

		if (NumDeclMembers == OverriddenCount)
		{
			ResetOverriddenPropertyNode(Root, EOverriddenPropertyOperation::Replace, {});
		}
	}
}

void RestoreStructOverrides(const UStruct* Struct, FStructView StructView, 
							FOverriddenPropertySet& Overrides, 
							const FRestoreContext& Ctx)
{
	FArchiveSerializedPropertyChain PropertyChain;

	RestoreStruct(Struct, StructView, Overrides, PropertyChain, Ctx);
	FlattenModifiedOverrides(Overrides.GetRootOverriddenPropertyNode(), StructView.Schema.Id, Ctx);
}

FScopeRestoreOverrides::FScopeRestoreOverrides(FOverriddenPropertySet* OverriddenProperties)
	: PrevOverrides(TlsRestoreOverrides)
{
	TlsRestoreOverrides = OverriddenProperties;
}

FScopeRestoreOverrides::~FScopeRestoreOverrides()
{
	TlsRestoreOverrides = PrevOverrides;
}

FOverriddenPropertySet* GetRestoreOverrides()
{
	return TlsRestoreOverrides;
}

} // namespace PlainProps::UE
