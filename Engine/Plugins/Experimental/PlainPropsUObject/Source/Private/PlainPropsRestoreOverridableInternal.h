// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsRead.h"
#include "PlainPropsTypes.h"

struct FOverriddenPropertySet;
struct FOverriddenPropertyNode;
enum class EOverriddenPropertyOperation : uint8;
class UStruct;

namespace PlainProps
{
	class FSchemaBindings;
	class FCustomBindings;
}

namespace PlainProps::UE
{
	struct FMetadataBindings;


struct FRestoreContext
{
	const FSchemaBindings& Schemas;
	FCustomBindings& Customs;
	const FMetadataBindings& Metadatas;
	TConstArrayView<FStructId> RuntimeIds;
};

void RestoreStructOverrides(const UStruct* Struct, FStructView StructView, FOverriddenPropertySet& Overrides, const FRestoreContext& Ctx);

void ResetOverriddenPropertyNode(FOverriddenPropertyNode& Node, EOverriddenPropertyOperation Op, TArray<FOverriddenPropertyNode> SubPropertyNodes = TArray<FOverriddenPropertyNode>());

class FScopeRestoreOverrides
{
public:
	FScopeRestoreOverrides(FOverriddenPropertySet* OverriddenProperties);
	~FScopeRestoreOverrides();

private:
	FOverriddenPropertySet* PrevOverrides;
};

FOverriddenPropertySet* GetRestoreOverrides();

} // namespace PlainProps::UE
