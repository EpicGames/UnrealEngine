// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsRoundtripTest.h"
#include "PlainPropsUObjectBindingsInternal.h"
#include "PlainPropsUObjectMetaBindingsInternal.h"
#include "PlainPropsUObjectDiffInternal.h"
#include "PlainPropsBuildSchema.h"
#include "PlainPropsLoadMember.h"
#include "PlainPropsUObjectRuntime.h"
#include "PlainPropsParse.h"
#include "PlainPropsPrint.h"
#include "PlainPropsSpecify.h"
#include "PlainPropsWrite.h"
#include "PlainPropsSaveOverridableInternal.h"
#include "PlainPropsRestoreOverridableInternal.h"
#include "Algo/Find.h"
#include "Async/ParallelFor.h"
#include "HAL/FileManager.h"
#include "JsonObjectGraph/Stringify.h"
#include "Logging/StructuredLog.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Serialization/Archive.h"
#include "Serialization/BufferWriter.h"
#include "Serialization/LargeMemoryReader.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/PackageStore.h"
#include "Serialization/UnversionedPropertySerializationInternal.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/LinkerLoad.h"
#include "UObject/LinkerSave.h"
#include "UObject/NameBatchSerialization.h"
#include "UObject/Package.h"
#include "UObject/SavePackage.h"
#include "UObject/TextProperty.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/OverridableManager.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

#define PP_WITH_ENGINE_HOOKS 0

namespace PlainProps::UE
{

struct FMemoryPropertyBatch
{
	TArray<FText> Texts; // Tricky to serialize intrusively

	static FSoleMemberSpec Spec(FName*, FDeclId Id)
	{
		return {Id, GUE.Members.Id, Specify<uint64>()};
	}

	static void Save(FMemberBuilder& Out, FName In, const FSaveContext&)
	{
		Out.Add(GUE.Members.Id, FSensitiveName(In).ToUnstableInt());
	}

	static void Load(FName& Out, FStructLoadView In)
	{
		Out = FSensitiveName::FromUnstableInt(LoadSole<FSensitiveName::IntType>(In)).ToName();
	}

	static FSoleMemberSpec Spec(FText*, FDeclId Id)
	{
		return {Id, GUE.Members.Id, Specify<int32>()};
	}

	void Save(FMemberBuilder& Out, const FText& In, const FSaveContext&)
	{
		Out.Add(GUE.Members.Id, Texts.Num());
		Texts.Add(In);
	}

	void Load(FText& Out, FStructLoadView In) const
	{
		Out = Texts[LoadSole<int32>(In)];
	}

	static FSoleMemberSpec Spec(FObjectHandle*, FDeclId Id)
	{
		return {Id, GUE.Members.Id, Specify<uint64>()};
	}

	static void Save(FMemberBuilder& Out, FObjectHandle In, const FSaveContext&)
	{
		static_assert(sizeof(In) == sizeof(uint64));
		Out.Add(GUE.Members.Id, reinterpret_cast<const uint64&>(In));
	}
	
	static constexpr void Plan(FObjectHandle*, FMemcpyLoadPlan& Out)
	{
		Out.Size = sizeof(FObjectHandle);
	}

	static FSoleMemberSpec Spec(FWeakObjectPtr*, FDeclId Id)
	{
		return {Id, GUE.Members.Id, Specify<uint64>()};
	}

	static void Save(FMemberBuilder& Out, const FWeakObjectPtr& In, const FSaveContext&)
	{
		// Save ObjectSerialNumber + ObjectIndex a single uint64
		static_assert(sizeof(FWeakObjectPtr) == sizeof(uint64));
		Out.Add(GUE.Members.Id, reinterpret_cast<const uint64&>(In));
	}

	static constexpr void Plan(FWeakObjectPtr*, FMemcpyLoadPlan& Out)
	{
		Out.Size = sizeof(FWeakObjectPtr);
	}

	static FSoleMemberSpec Spec(FSoftObjectPtr*, FDeclId Id)
	{
		return {Id, GUE.Members.Id, FMemberSpec(GUE.Structs.SoftObjectPath)};
	}

	static void Save(FMemberBuilder& Out, const FSoftObjectPtr& In, const FSaveContext& Ctx)
	{
		FBuiltStruct* SoftPath = SaveStruct(&In.GetUniqueID(), GUE.Structs.SoftObjectPath, Ctx);
		Out.AddStruct(GUE.Members.Id, GUE.Structs.SoftObjectPath, SoftPath);
	}

	static void Load(FSoftObjectPtr& Out, FStructLoadView In)
	{
		Out.ResetWeakPtr();
		LoadSoleStruct(&Out.GetUniqueID(), In);
	}
	
	static FSoleMemberSpec Spec(FLazyObjectPtr*, FDeclId Id)
	{
		return {Id, GUE.Members.Id, FMemberSpec(GUE.Structs.Guid)};
	}
	
	static void Save(FMemberBuilder& Out, const FLazyObjectPtr& In, const FSaveContext& Ctx)
	{
		FBuiltStruct* Guid = SaveStruct(&In.GetUniqueID(), GUE.Structs.Guid, Ctx);
		Out.AddStruct(GUE.Members.Id, GUE.Structs.Guid, Guid);
	}

	static void Load(FLazyObjectPtr& Out, FStructLoadView In)
	{
		Out.ResetWeakPtr();
		LoadSoleStruct(&Out.GetUniqueID(), In);
	}
};

inline bool DiffProperty(FName A, FName B) { return !A.IsEqual(B, ENameCase::CaseSensitive); }
inline bool DiffProperty(const FText& A, const FText& B) { return !FTextProperty::Identical_Implementation(A, B, 0); }
inline bool DiffProperty(FObjectHandle A, FObjectHandle B) { return A != B; }
inline bool DiffProperty(const FSoftObjectPath& A, const FSoftObjectPath& B) { return A != B; }
inline bool DiffProperty(const FWeakObjectPtr& A, const FWeakObjectPtr& B) { return A != B; }
inline bool DiffProperty(const FSoftObjectPtr& A, const FSoftObjectPtr& B) { return A != B; }
inline bool DiffProperty(const FLazyObjectPtr& A, const FLazyObjectPtr& B) { return A != B; }

template<class Type, class BatchType>
struct TCustomPropertyBinding final : ICustomBinding
{
	static constexpr bool bFastLoad = requires(Type* Tag, FMemcpyLoadPlan& Out) { BatchType::Plan(Tag, Out); };

	TCustomPropertyBinding(BatchType& InBatch) : Batch(InBatch) {}

	BatchType& Batch;

	virtual void SaveCustom(FMemberBuilder& Dst, const void* Src, FBaseline Base, const FSaveContext& Ctx) override
	{
		const void* Default = Base.Get();
		
		// Do not perform a diff if we are saving via the override system because
		// we want to save despite the value being the same as the default.
		if (!Default || (GetSaveOverrides() != nullptr) || DiffCustom(Src, Default, Ctx))
		{
			Batch.Save(Dst, *static_cast<const Type*>(Src), Ctx);
		}
	}

	virtual void LoadCustom(void* Dst, FStructLoadView Src, ECustomLoadMethod) const override
	{
		if constexpr (bFastLoad)
		{
			unimplemented();
		}
		else
		{
			Batch.Load(*static_cast<Type*>(Dst), Src);
		}
	}

	virtual bool DiffCustom(const void* A, const void* B, const FBindContext&) const override
	{
		return DiffProperty(*static_cast<const Type*>(A), *static_cast<const Type*>(B));
	}

	virtual void PlanCustom(FMemcpyLoadPlan& Out) const override
	{
		if constexpr (bFastLoad)
		{
			Batch.Plan((Type*)nullptr, Out);
		}
	}
};

template<class BatchType>
struct TCustomPropertyBindings
{
	TCustomPropertyBindings(BatchType& Batch, const FCustomBindings& Underlay)
	: Overlay(Underlay)
	, Name(Batch)
	, Text(Batch)
	, ObjectPtr(Batch)
	, SoftObjectPtr(Batch)
	, WeakObjectPtr(Batch)
	, LazyObjectPtr(Batch)
	{
		Bind(GUE.Structs.Name, Name);
		Bind(GUE.Structs.Text, Text);
		Bind(GUE.Structs.ClassPtr, ObjectPtr); // TSubclassOf<> is essentially a TObjectPtr
		Bind(GUE.Structs.ObjectPtr, ObjectPtr);
		Bind(GUE.Structs.SoftObjectPtr, SoftObjectPtr);
		Bind(GUE.Structs.WeakObjectPtr, WeakObjectPtr);
		Bind(GUE.Structs.LazyObjectPtr, LazyObjectPtr);
	}

	template<class Type>
	void Bind(FDualStructId Id, TCustomPropertyBinding<Type, BatchType>& Binding)
	{
		Overlay.BindStruct(Id, Binding, Declare(BatchType::Spec((Type*)nullptr, Id)), {});
	}

	FCustomBindingsOverlay								Overlay;
	TCustomPropertyBinding<FName, BatchType>			Name;
	TCustomPropertyBinding<FText, BatchType>			Text;
	TCustomPropertyBinding<FObjectHandle, BatchType>	ObjectPtr;
	TCustomPropertyBinding<FSoftObjectPtr, BatchType>	SoftObjectPtr;
	TCustomPropertyBinding<FWeakObjectPtr, BatchType>	WeakObjectPtr;
	TCustomPropertyBinding<FLazyObjectPtr, BatchType>	LazyObjectPtr;
};

////////////////////////////////////////////////////////////////////////////////////////////////

struct FMemoryBatch
{
	TArray64<uint8>				Data;
	TArray<FStructId>			RuntimeIds; // To avoid reindexing schema FType
	FMemoryPropertyBatch		Properties; // Contains FTexts referenced from Data
};

static constexpr uint32 Magics[] = { 0xFEEDF00D, 0xABCD1234, 0xDADADAAA, 0x99887766, 0xF0F1F2F3 };
volatile static const UObject* GDebugNoteObject;

class FBatchSaver
{
public:
	FBatchSaver(FCustomBindings& Customs, int32 NumReserve)
	: FlatCtx{{GUE.Schemas, Customs}, Scratch, nullptr}
	, DeltaCtx{{GUE.Schemas, Customs}, Scratch, &GUE.Defaults}
	{
		SavedObjects.Reserve(NumReserve);
		ObjectMap.Reserve(NumReserve);
	}

	// If Arch, delta-serialize the matching part of the object inheritance chain
	FBuiltStruct* SaveImpl(FBindId Id, const UStruct* Class, const UObject* Object, const UStruct* ArchClass, const UObject* Arch)
	{
		if (ArchClass == nullptr)
		{
			return SaveStruct(Object, Id, FlatCtx);
		}
		else if (Class == ArchClass)
		{
			return SaveStructDelta(Object, FBaseline(Arch), Id, DeltaCtx);
		}

		// Archetype is some super class, save derived part(s) with null archetype
		check(Class->IsChildOf(ArchClass));
		TArray<FDefaultLink, TInlineAllocator<4>> Chain;
		const UStruct* SuperIt = Class;
		while ((SuperIt = SkipEmptyBases(SuperIt->GetInheritanceSuper())))
		{
			Chain.Emplace(nullptr);
			if (SuperIt == ArchClass)
			{
				Chain.Emplace(Arch);
				check(Object->HasAllFlags(RF_ClassDefaultObject));
				check(!Object->GetClass()->HasAllClassFlags(CLASS_Native));
				return SaveStructDelta(Object, FBaseline(Chain.GetData(), Chain.Num()), Id, DeltaCtx);
			}
		}

		checkf(!SkipEmptyBases(ArchClass), TEXT("Code missing for skipped ArchClass without skipping all ArchClass's supers"));

		// ArchClass was skipped so skip delta serialization, but use
		// DeltaCtx for IDefaultStructs lookup of range-of-structs
		return SaveStruct(Object, Id, DeltaCtx);
	}

	void Save(FBindId Id, const UObject* Object, const UObject* Arch)
	{
		FBuiltStruct* Built;
		if (Arch && ShouldBind(Arch->GetClass()))
		{
			const UStruct* Class = SkipEmptyBases(Object->GetClass());
			const UStruct* ArchClass = SkipEmptyBases(Arch->GetClass());
			Built = SaveImpl(Id, Class, Object, ArchClass, Arch);
		}
		else
		{
			Built = SaveImpl(Id, Object->GetClass(), Object, nullptr, nullptr);
		}
		ObjectMap.Add(SavedObjects.Num());
		SavedObjects.Emplace(Id, Built, Object);
	}

	void SaveMeta(FBindId Id, const UStruct* Struct)
	{
		FBuiltStruct* Built = SaveStruct(Struct, Id, FlatCtx);
		ObjectMap.Add(SavedObjects.Num());
		SavedObjects.Emplace(Id, Built, Struct);
	}

	void SaveObjectWithOverrides(FBindId Id, const UObject* Object, 
								 const FOverriddenPropertySet& Overriddes, const UObject* Arch)
	{
		FBuiltStruct* Built = SaveStructOverrides(Overriddes, FBaseline(Arch), Id, DeltaCtx);
		
		ObjectMap.Add(SavedObjects.Num());
		SavedObjects.Emplace(Id, Built, Object);
	}
	
	void Skip(FBindId Id, const UObject* Object)
	{
		ObjectMap.Add(~0u);
	}

	// Use existing SaveNameBatch() API for now, can be optimized later
	void WriteStableNames(TArray64<uint8>& Out, TConstArrayView<FNameId> UsedNames) const
	{
		TArray<FDisplayNameEntryId> Entries;
		TArray<int32> Numbers;
		Entries.Reserve(UsedNames.Num());
		Numbers.Reserve(UsedNames.Num());
		for (FNameId NameId : UsedNames)
		{
			FSensitiveName SensitiveName = GUE.Names.ResolveName(NameId);
			FName Name = SensitiveName.ToName();
			Entries.Add(FDisplayNameEntryId(Name));
			Numbers.Add(Name.GetNumber());
		}

		FBufferWriter NameMapArchive(nullptr, 0, EBufferWriterFlags::AllowResize | EBufferWriterFlags::TakeOwnership);
		SaveNameBatch(Entries, NameMapArchive);

		WriteNumAndArray(Out, TArrayView<uint8>(static_cast<uint8*>(NameMapArchive.GetWriterData()), static_cast<int32>(NameMapArchive.Tell())));
		WriteNumAndArray(Out, Numbers);
	}

	TArray64<uint8>	Write(TArray<FStructId>* OutRuntimeIds = nullptr) const
	{
		ESchemaFormat Format = OutRuntimeIds ? ESchemaFormat::InMemoryNames : ESchemaFormat::StableNames;

		// Build partial schemas
		const FBindDeclarations Types(GUE.Enums, FlatCtx.Customs, GUE.Schemas);
		FSchemasBuilder SchemaBuilders(GUE.Names, Types, Scratch, Format);
		for (const FSavedObject& Object : SavedObjects)
		{
			GDebugNoteObject = Object.Input;
			SchemaBuilders.NoteStructAndMembers(Object.Id, *Object.Built);
		}
		GDebugNoteObject = nullptr;
		FBuiltSchemas Schemas = SchemaBuilders.Build();

		// Save schema ids on the side when using InMemoryNames
		if (OutRuntimeIds)
		{
			*OutRuntimeIds = ExtractRuntimeIds(Schemas);
		}

		FWriter Writer(GUE.Names, Types, Schemas, Format);
		TArray64<uint8> Out;

		WriteInt(Out, Magics[0]);
		// Write out FNames when using StableNames
		if (!OutRuntimeIds)
		{
			WriteStableNames(Out, Writer.GetUsedNames());
		}

		// Write schemas
		WriteInt(Out, Magics[1]);
		WriteAlignmentPadding<uint32>(Out);
		TArray64<uint8> Tmp;
		Writer.WriteSchemas(/* Out */ Tmp);
		WriteNumAndArray(Out, TArrayView<const uint8, int64>(Tmp));
		Tmp.Reset();

		// Write objects
		WriteInt(Out, Magics[2]);
		for (const FSavedObject& Object : SavedObjects)
		{
			WriteInt(/* out */ Tmp, Magics[3]);
			WriteInt(/* out */ Tmp, Writer.GetWriteId(Object.Id).Get().Idx);
			Writer.WriteMembers(/* out */ Tmp, Object.Id, *Object.Built);
			WriteSkippableSlice(Out, Tmp);
			Tmp.Reset();
		}

		// Write object terminator
		WriteSkippableSlice(Out, TConstArrayView64<uint8>());
		WriteInt(Out, Magics[4]);

		// Write object map that allows for empty objects/(indices) without built data
		WriteNumAndArray(Out, ObjectMap);
		
		return Out;
	}

private:
	struct FSavedObject
	{
		FBindId				Id;
		FBuiltStruct*		Built;
		const UObject*		Input; // For debug
	};

	TArray<FSavedObject>		SavedObjects;
	TArray<uint32>				ObjectMap;
	mutable FScratchAllocator	Scratch;
	FSaveContext				FlatCtx;
	FSaveContext				DeltaCtx;

	template<typename ArrayType>
	static void WriteNumAndArray(TArray64<uint8>& Out, const ArrayType& Items)
	{
		WriteInt(Out, IntCastChecked<uint32>(Items.Num()));
		WriteArray(Out, Items);
	}
};

class FMemoryBatchLoader
{
public:
	FMemoryBatchLoader(const FCustomBindings& Customs, FMemoryView Data, TConstArrayView<FStructId> RuntimeIds)
	{
		// Read and mount schemas
		FByteReader It(Data);
		verify(It.Grab<uint32>() == Magics[0]);
		verify(It.Grab<uint32>() == Magics[1]);
		It.SkipAlignmentPadding<uint32>();
		uint32 SchemasSize = It.Grab<uint32>();
		const FSchemaBatch* SavedSchemas = ValidateSchemas(It.GrabSlice(SchemasSize));
		verify(It.Grab<uint32>() == Magics[2]);
		
		FSchemaBatchId Batch = MountReadSchemas(SavedSchemas);

		// Read objects
 		while (uint64 NumBytes = It.GrabVarIntU())
		{	
			FByteReader ObjIt(It.GrabSlice(NumBytes));
			verify(ObjIt.Grab<uint32>() == Magics[3]);
			FStructSchemaId Id = { ObjIt.Grab<uint32>() };
			check(Id != FStructSchemaId{});
			Objects.Add({ { Id, Batch }, ObjIt });
		}
		
		verify(It.Grab<uint32>() == Magics[4]);
		verify(GrabNumAndArray<uint32>(It).Num() == Objects.Num());
		It.CheckEmpty();
		check(!Objects.IsEmpty());

		// Finally create load plans
		Plans = CreateLoadPlans(Batch, Customs, GUE.Schemas, RuntimeIds, ESchemaFormat::InMemoryNames);
	}

	~FMemoryBatchLoader()
	{
		check(LoadIdx == Objects.Num()); // Test should load all saved objects
		Plans.Reset();
		UnmountReadSchemas(Objects[0].Schema.Batch);
	}

	FORCENOINLINE void Load(UObject* Dst)
	{
		FStructView In = Objects[LoadIdx];
		LoadStruct(Dst, In.Values, In.Schema.Id, *Plans);
		++LoadIdx;
	}

	FORCENOINLINE void Reload(UObject* Dst, int32 ReloadIdx)
	{
		FStructView In = Objects[ReloadIdx];
		LoadStruct(Dst, In.Values, In.Schema.Id, *Plans);
	}

private:
	FLoadBatchPtr				Plans;
	TArray<FStructView>			Objects;
	int32						LoadIdx = 0;
	
	template<typename T>
	static TConstArrayView<T> GrabNumAndArray(/* in-out */ FByteReader& It)
	{
		uint32 Num = It.Grab<uint32>();
		return MakeArrayView(reinterpret_cast<const T*>(It.GrabBytes(Num * sizeof(T))), Num);
	}
};

class FPersistentBatchLoader
{
public:
	FPersistentBatchLoader(const FCustomBindings& InCustoms, FMemoryView Data, TPair<FSensitiveName, FSensitiveName> RemapNames)
	: Customs(InCustoms)
	{
		// Read ids
		FByteReader It(Data);
		verify(It.Grab<uint32>() == Magics[0]);
		ReadStableNames(It);

		// Remap original/saved names to load/instance runtime names,
		// for now if the package name of the batch itself has been saved (e.g. for a blueprint class scope name),
		// then replace it with its load/instance package name
		if (RemapNames.Key != RemapNames.Value)
		{
			if (FSensitiveName* Name = Algo::Find(Names, FSensitiveName(RemapNames.Key)))
			{
				*Name = FSensitiveName(RemapNames.Value);
			}
		}

		// Read and mount schemas
		verify(It.Grab<uint32>() == Magics[1]);
		It.SkipAlignmentPadding<uint32>();
		uint32 SchemasSize = It.Grab<uint32>();
		const FSchemaBatch* SavedSchemas = ValidateSchemas(It.GrabSlice(SchemasSize));
		verify(It.Grab<uint32>() == Magics[2]);
		
		// Bind saved ids to runtime ids, make new schemas with new ids and mount them
		FIdTranslator RuntimeIds(GUE.Names, MakeConstArrayView(Names), *SavedSchemas);
		TranslatedSchemas = CreateTranslatedSchemas(*SavedSchemas, RuntimeIds.Translation);
		FSchemaBatchId Batch = MountReadSchemas(TranslatedSchemas);

		// Read objects
 		while (uint64 NumBytes = It.GrabVarIntU())
		{	
			FByteReader ObjIt(It.GrabSlice(NumBytes));
			verify(ObjIt.Grab<uint32>() == Magics[3]);
			FStructSchemaId Id = { ObjIt.Grab<uint32>() };
			Objects.Add({ { Id, Batch }, ObjIt });
		}
		
		verify(It.Grab<uint32>() == Magics[4]);
		ObjectMap = GrabNumAndArray<uint32>(It);
		check(ObjectMap.Num() >= Objects.Num());
		check(!Objects.IsEmpty());

		// Finally create load plans
		LoadStructIds = RuntimeIds.Translation.GetStructIds(NumStructSchemas(Batch));
		Plans = CreateLoadPlans(Batch, Customs, GUE.Schemas, LoadStructIds, ESchemaFormat::StableNames);
	}

	void ReadStableNames(FByteReader& It)
	{
		TConstArrayView64<uint8> NameBatch = GrabNumAndArray<uint8>(It);
		TConstArrayView<int32> NameNumbers = GrabNumAndArray<int32>(It);
		FMemoryReaderView Ar(MakeArrayView(NameBatch.GetData(), NameBatch.Num()));
		TArray<FDisplayNameEntryId> NameEntries = LoadNameBatch(Ar);
		Names.Reserve(NameEntries.Num());
		check(NameEntries.Num() == NameNumbers.Num());
		for (int64 Idx = 0, Num = NameEntries.Num(); Idx < Num; ++Idx)
		{
			Names.Emplace(NameEntries[Idx].ToName(NameNumbers[Idx]));
		}
	}

	~FPersistentBatchLoader()
	{
		Plans.Reset();
		UnmountReadSchemas(Objects[0].Schema.Batch);
	}

	bool HasObject(int32 ObjectIndex)
	{
		return ObjectMap[ObjectIndex] != ~0u;
	}

	FORCENOINLINE void LoadObject(UObject* Dst, int32 ObjectIndex)
	{
		check(HasObject(ObjectIndex));
		FStructView In = Objects[ObjectMap[ObjectIndex]];
		if (!HasLoadPlan(In.Schema.Id, *Plans))
		{
			UpdateLoadPlans(*Plans, Customs, GUE.Schemas, LoadStructIds);
			++NumUpdates;
		}
		LoadStruct(Dst, In.Values, In.Schema.Id, *Plans);
		++NumLoaded;
	}

	FORCENOINLINE void RestoreObjectOverrides(UObject* Object, int32 ObjectIndex, FOverriddenPropertySet& Overrides)
	{
		check(HasObject(ObjectIndex));
		FStructView StructView = Objects[ObjectMap[ObjectIndex]];

		FRestoreContext Ctx{ GUE.Schemas, GUE.Customs, GUE.Metadatas, MakeConstArrayView(LoadStructIds) };

		RestoreStructOverrides(Object->GetClass(), StructView, Overrides, Ctx);
	}

	static void PrintDebugInfo()
	{
		UE_LOGFMT(LogPlainPropsUObject, Display,
			"Loaded {NumLoaded} exports, {NumUpdates} load plan updates", NumLoaded, NumUpdates);
	}

private:
	static inline int32			NumLoaded = 0;
	static inline int32			NumUpdates = 0;

	const FCustomBindings&		Customs;
	const FSchemaBatch*			TranslatedSchemas = nullptr;
	TArray<FSensitiveName>		Names;
	TArray<FStructId>			LoadStructIds;
	FLoadBatchPtr				Plans;
	TArray<FStructView>			Objects;
	TConstArrayView<uint32>		ObjectMap;

	template<typename T>
	static TConstArrayView<T> GrabNumAndArray(/* in-out */ FByteReader& It)
	{
		uint32 Num = It.Grab<uint32>();
		return MakeArrayView(reinterpret_cast<const T*>(It.GrabBytes(Num * sizeof(T))), Num);
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////

// Similar to PlainProps FMemoryBatch but for storing FArchive property serialization results
struct FArchivedProperties
{
	TArray<uint8>			Data;
	TArray<FText>			Texts;
};

static constexpr uint32 RoundtripPortFlags = PPF_UseDeprecatedProperties | PPF_ForceTaggedSerialization;

// Match FMemoryPropertyBatch somewhat for a fair comparison, e.g. save FText on side and FName as integer
class FPropertyWriter final : public FMemoryWriter
{
	TArray<FText>& Texts;
public:
	FPropertyWriter(FArchivedProperties& Out) 
	: FMemoryWriter(Out.Data)
	, Texts(Out.Texts)
	{
		SetPortFlags(RoundtripPortFlags);
	}

	FORCENOINLINE void WriteProperties(UObject* Object, UObject* Defaults)
	{
		UClass* Class = Object->GetClass();
		// See DiffClass in UObject::SerializeScriptProperties
		UClass* DefaultsClass = Object->HasAnyFlags(RF_ClassDefaultObject) ? Class->GetSuperClass() : Class;
		Class->SerializeTaggedProperties(*this, reinterpret_cast<uint8*>(Object), DefaultsClass, reinterpret_cast<uint8*>(Defaults));
	}

	virtual FArchive& operator<<(FText& Value) override
	{
		int32 Idx = INDEX_NONE;
		if (!Value.IsEmpty())
		{
			Idx = Texts.Num();
			Texts.Add(Value);
		}
		return WriteValue(Idx);
	}
	virtual FArchive& operator<<(FName& Value) override				{ return WriteValue(FSensitiveName(Value).ToUnstableInt());}
	virtual FArchive& operator<<(UObject*& Value) override			{ return WriteValue(reinterpret_cast<uint64&>(Value)); }
	virtual FArchive& operator<<(FObjectPtr& Value) override		{ return WriteValue(reinterpret_cast<uint64&>(Value)); }
	virtual FArchive& operator<<(FWeakObjectPtr& Value) override	{ return WriteValue(reinterpret_cast<uint64&>(Value)); }
	virtual FArchive& operator<<(FLazyObjectPtr& Value) override	{ return WriteValue(Value.GetUniqueID()); }
	virtual FArchive& operator<<(FSoftObjectPtr& Value) override	{ Value.GetUniqueID().SerializePath(*this); return *this; }
	virtual FArchive& operator<<(FSoftObjectPath& Value) override	{ Value.SerializePath(*this); return *this; }
	virtual FString GetArchiveName() const override					{ return "FPropertyWriter"; }

	template<typename T>
	FArchive& WriteValue(T Value)
	{
		return *this << Value;
	}
};

class FPropertyReader final : public FMemoryReader
{
	const TArray<FText>& Texts;
public:
	FPropertyReader(const FArchivedProperties& Out)
	: FMemoryReader(Out.Data)
	, Texts(Out.Texts)
	{
		SetPortFlags(RoundtripPortFlags);
	}

	FORCENOINLINE void ReadProperties(UObject* Object, UObject* Defaults)
	{
		UClass* Class = Object->GetClass();
		// See DiffClass in UObject::SerializeScriptProperties
		UClass* DefaultsClass = Object->HasAnyFlags(RF_ClassDefaultObject) ? Class->GetSuperClass() : Class;
		Class->SerializeTaggedProperties(*this, reinterpret_cast<uint8*>(Object), DefaultsClass, reinterpret_cast<uint8*>(Defaults));
	}

	virtual FArchive& operator<<(FText& Value) override
	{ 
		int32 Idx = ReadValue<int32>();
		Value = Idx == INDEX_NONE ? FText::GetEmpty() : Texts[Idx];
		return *this;
	}

	virtual FArchive& operator<<(FName& Value) override
	{
		Value = FSensitiveName::FromUnstableInt(ReadValue<FSensitiveName::IntType>()).ToName();
		return *this;
	}

	virtual FArchive& operator<<(UObject*& Value) override			{ reinterpret_cast<uint64&>(Value) = ReadValue<uint64>(); return *this; }
	virtual FArchive& operator<<(FObjectPtr& Value) override		{ reinterpret_cast<uint64&>(Value) = ReadValue<uint64>(); return *this; }
	virtual FArchive& operator<<(FWeakObjectPtr& Value) override	{ reinterpret_cast<uint64&>(Value) = ReadValue<uint64>(); return *this; }
	virtual FArchive& operator<<(FLazyObjectPtr& Value) override	{ Value.ResetWeakPtr(); Value.GetUniqueID() = ReadValue<FUniqueObjectGuid>(); return *this; }
	virtual FArchive& operator<<(FSoftObjectPtr& Value) override	{ Value.ResetWeakPtr(); Value.GetUniqueID().SerializePath(*this); return *this; }
	virtual FArchive& operator<<(FSoftObjectPath& Value) override	{ Value.SerializePath(*this); return *this; }
	virtual FString GetArchiveName() const override					{ return "FPropertyReader"; }

	template<typename T>
	T ReadValue()
	{
		T Out;
		*this << Out;
		return Out;
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////

struct FInstance
{
	FString PathName;
	UObject* Orig;
	UObject* Arch;
	UObject* Base;
	UObject* PP;
	UObject* TPS;
	UObject* UPS;
	FBindId Id;

	void Init()
	{
		UClass* Class = Orig->GetClass();
		Class->GetDefaultObject(/* create lazily */ true);
		Arch = Orig->GetArchetype();
		check(Arch != Orig);

		if (Arch->GetClass() != Class)
		{
			Arch->GetClass()->GetDefaultObject(/* create lazily */ true);
		}

		Id = GUE.Names.IndexBindId(IndexStruct(Class));
	}
};

static UObject* MakeEmptyInstance(UObject* Obj, FName Name)
{
	FStaticConstructObjectParameters Params(Obj->GetClass());
	Params.Outer = Obj->GetOuter();//GetTransientPackage();
	Params.Name = Name;
	Params.SetFlags = Obj->GetFlags();
	Params.Template = Obj->GetArchetype();
	Params.ExternalPackage = (Params.SetFlags & RF_HasExternalPackage) ? Obj->GetExternalPackage() : nullptr;
	Params.bAssumeTemplateIsArchetype = true;
	Params.bCopyTransientsFromClassDefaults = !(Params.SetFlags & RF_ClassDefaultObject);
	UObject* Result = StaticConstructObject_Internal(Params);

	// When creating CDOs the name passed-in will be ignored and the default name will be used instead.
	// (e.g., Default__MyType). To avoid reusing the same instance we force a rename.
	if (Result->GetFName() != Name)
	{
		Result->Rename(*Name.ToString(), nullptr, REN_DontCreateRedirectors | REN_DoNotDirty | REN_NonTransactional);
	}

	return Result;
}

static bool IncludeClass(UClass* Class)
{
	static const FName Exclusions[] = {
		"CitySampleUnrealEdEngine", // Cloning MTAccessDetector crash
		"GameFeaturePluginStateMachine", // Cloning ensure
		"WorldSettings", // QAGame
		// CitySample - Enum value 8 is undeclared in /Script/Engine.ERichCurveTangentMode, illegal value detected in /Script/Engine.RichCurveKey::TangentMode
		// Enum value 7 is undeclared in /Script/Text3D.EText3DFontStyleFlags, illegal value detected in /Script/Text3DEditor.Text3DEditorFont::FontStyleFlags
		// FontStyleFlags	Monospace | Bold | Italic (7 '\a')	UnrealEditor-Text3DEditor.dll!EText3DFontStyleFlags
		"Text3DEditorFontSubsystem",
	};
	static const FName SuperExclusions[] = { "LevelScriptActor" };

	if (!!Algo::Find(Exclusions, Class->GetFName()))
	{
		return false;
	}

	// Exclude IDOs
	static constexpr EClassFlags IdoFlags = CLASS_NotPlaceable | CLASS_Hidden | CLASS_HideDropDown;
	if (Class->HasAllClassFlags(IdoFlags))
	{
		return false;
	}

	for (UStruct* Super = Class->GetInheritanceSuper(); Super; Super = Super->GetInheritanceSuper())
	{
		if (!!Algo::Find(SuperExclusions, Super->GetFName()))
		{
			return false;
		}
	}
	return true;
}

FORCENOINLINE static void SavePlainProps(FBatchSaver& Batch, TConstArrayView<FInstance> Instances)
{
	for (const FInstance& Instance : Instances)
	{
		Batch.Save(Instance.Id, Instance.Orig, Instance.Arch);
	}
}

FORCENOINLINE static void LoadPlainProps(FMemoryBatchLoader& Batch, TConstArrayView<FInstance> Instances)
{
	for (const FInstance& Instance : Instances)
	{	
		Batch.Load(Instance.PP);
	}
}

template<bool bUps>
FORCENOINLINE void SaveArchive(FPropertyWriter& Archive, TConstArrayView<FInstance> Instances)
{
	Archive.SetUseUnversionedPropertySerialization(bUps);
	for (const FInstance& Instance : Instances)
	{
		Archive.WriteProperties(Instance.Orig, Instance.Arch);
	}
}

template<bool bUps>
FORCENOINLINE void LoadArchive(FPropertyReader& Archive, TConstArrayView<FInstance> Instances)
{
	Archive.SetUseUnversionedPropertySerialization(bUps);
	for (const FInstance& Instance : Instances)
	{
		Archive.ReadProperties(bUps ? Instance.UPS : Instance.TPS, Instance.Arch);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////

enum class EMarkupFormat { Yaml, Json };

class FStableNameBatchIds final : public FStableBatchIds
{
	TConstArrayView<FSensitiveName> Names;
public:
	FStableNameBatchIds(FSchemaBatchId Batch, TConstArrayView<FSensitiveName> InNames) : FStableBatchIds(Batch), Names(InNames) {}
	using FStableBatchIds::AppendString;

	virtual uint32				NumNames() const override							{ return static_cast<uint32>(Names.Num()); }
	virtual void				AppendString(FUtf8Builder& Out, FNameId Name) const override { Names[Name.Idx].AppendString(Out); }
};

[[nodiscard]] static FSchemaBatchId ParseBatch(
	TArray64<uint8>& OutData,
	TArray<FStructView>& OutObjects,
	FUtf8StringView MarkupView,
	EMarkupFormat MarkupFormat=EMarkupFormat::Yaml)
{
	// Parse markup
	switch (MarkupFormat)
	{
	case EMarkupFormat::Yaml: ParseYamlBatch(OutData, MarkupView); break;
	case EMarkupFormat::Json: ParseJsonBatch(OutData, MarkupView); break;
	}

	// Grab and mount parsed schemas
	FByteReader It(MakeMemoryView(OutData));
	const uint32 SchemasSize = It.Grab<uint32>();
	FMemoryView SchemasView = It.GrabSlice(SchemasSize);
	const FSchemaBatch* Schemas = ValidateSchemas(SchemasView);
	FSchemaBatchId Batch = MountReadSchemas(Schemas);

	// Grab parsed objects
	while (uint64 NumBytes = It.GrabVarIntU())
	{	
		FByteReader ObjIt(It.GrabSlice(NumBytes));
		FStructSchemaId Schema = { ObjIt.Grab<uint32>() };
		OutObjects.Add({ { Schema, Batch }, ObjIt });
	}
	
	return Batch;
}

static void RoundtripText(
	const FBatchIds& BatchIds,
	TConstArrayView<FStructView> Objects,
	TConstArrayView<FInstance> Instances,
	ESchemaFormat Format,
	FStringView AssetName = {},
	EMarkupFormat MarkupFormat = EMarkupFormat::Yaml)
{
	// Print markup
	UE_LOGFMT(LogPlainPropsUObject, Display, "Printing to PlainProps text using {Format}...", ToString(Format));
	TUtf8StringBuilder<256> Markup;
	Markup.Reserve(INT_MAX);
	switch (MarkupFormat)
	{
	case EMarkupFormat::Yaml: PrintYamlBatch(Markup, BatchIds, Objects); break;
	case EMarkupFormat::Json: PrintJsonBatch(Markup, BatchIds, Objects); break;
	}
	FUtf8StringView MarkupView = Markup.ToView();

	// Write to file
	if (AssetName.Len() == 0)
	{
		AssetName = (Format == ESchemaFormat::InMemoryNames ? TEXTVIEW("DiffTest_InMemoryNames") : TEXTVIEW("DiffTest_StableNames"));
	}
	const TCHAR* Extension = (MarkupFormat == EMarkupFormat::Yaml) ? TEXT(".yaml") : TEXT(".json");
	const FString MarkupPath = FPaths::Combine(FPaths::ProjectSavedDir(), TEXT("PlainProps"), AssetName) + Extension;
	if (TUniquePtr<FArchive> FileWriter(IFileManager::Get().CreateFileWriter(*MarkupPath)); FileWriter)
	{
		UE_LOGFMT(LogPlainPropsUObject, Display, "Writing {KB}KB {Format} as {MarkupPath}...", Markup.Len() >> 10, Extension + 1, *MarkupPath);
		FileWriter->Serialize((void*)MarkupView.GetData(), MarkupView.Len());
	}

	// Parse text markup
	UE_LOGFMT(LogPlainPropsUObject, Display, "Parsing PlainProps text using {Format}...", ToString(Format));
	TArray64<uint8> Data;
	TArray<FStructView> ParsedObjects;
	FSchemaBatchId ParsedBatch = ParseBatch(Data, ParsedObjects, MarkupView, MarkupFormat);

	if (Format == ESchemaFormat::StableNames && Instances.Num())
	{
		UE_LOGFMT(LogPlainPropsUObject, Display, "Diffing PlainProps parsed objects using {Format}...", ToString(Format));

		// Diff schemas
		check(!DiffSchemas(BatchIds.GetBatchId(), ParsedBatch));

		// Diff objects
		check(Objects.Num() == ParsedObjects.Num());
		check(Objects.Num() == Instances.Num());
		const int32 NumObjects = FMath::Min(Objects.Num(), ParsedObjects.Num());
		uint32 NumDiffs = 0;
		TUtf8StringBuilder<256> Diffs;
		for (int32 I = 0; I < NumObjects; ++I)
		{
			FStructView In = Objects[I];
			FStructView Parsed = ParsedObjects[I];
			FReadDiffPath DiffPath;
			if (DiffStruct(In, Parsed, DiffPath))
			{
				PrintDiff(Diffs, BatchIds, DiffPath);
				Diffs.Append(" in ");
				Diffs.Append(Instances[I].PathName);
				Diffs.Append("\n");
				++NumDiffs;
			}
		}
		UE_LOGFMT(LogPlainPropsUObject, Display,
			"Detected {Diffs} diffs in {Objs} PlainProps parsed objects from {KB}KB {Format} text using StableNames\n{Diffs}",
			NumDiffs, NumObjects, Markup.Len() >> 10, Extension + 1, Diffs.ToString());
	}

	// Unmount parsed schemas
	UnmountReadSchemas(ParsedBatch);
}

class FBatchTextRoundtripper
{
public:
	FBatchTextRoundtripper(FMemoryView Data, ESchemaFormat InFormat, EMarkupFormat InMarkupFormat)
	: Format(InFormat)
	, MarkupFormat(InMarkupFormat)
	{
		FByteReader It(Data);
		verify(It.Grab<uint32>() == Magics[0]);

		// Read FNames when using Stable Names
		if (Format == ESchemaFormat::StableNames)
		{
			ReadStableNames(It);
		}
		
		// Read and mount schemas
		verify(It.Grab<uint32>() == Magics[1]);
		It.SkipAlignmentPadding<uint32>();
		const uint32 SchemasSize = It.Grab<uint32>();
		FMemoryView SavedSchemasView = It.GrabSlice(SchemasSize);
		const FSchemaBatch* SavedSchemas = ValidateSchemas(SavedSchemasView);
		verify(It.Grab<uint32>() == Magics[2]);
		FSchemaBatchId Batch = MountReadSchemas(SavedSchemas);
		
		// Read objects
		while (uint64 NumBytes = It.GrabVarIntU())
		{	
			FByteReader ObjIt(It.GrabSlice(NumBytes));
			verify(ObjIt.Grab<uint32>() == Magics[3]);
			FStructSchemaId Id = { ObjIt.Grab<uint32>() };
			Objects.Add({ { Id, Batch }, ObjIt });
		}
		
		verify(It.Grab<uint32>() == Magics[4]);
		check(!Objects.IsEmpty());

		// Create BatchIds 
		if (Format == ESchemaFormat::StableNames)
		{
			BatchIds = MakeUnique<FStableNameBatchIds>(Batch, Names);
		}
		else
		{
			BatchIds = MakeUnique<FMemoryBatchIds>(Batch, GUE.Names);
		}
	}

	void ReadStableNames(FByteReader& It)
	{
		TConstArrayView<uint8> NameBatch = GrabNumAndArray<uint8>(It);
		TConstArrayView<int32> Numbers = GrabNumAndArray<int32>(It);
		FMemoryReaderView Ar(NameBatch);
		TArray<FDisplayNameEntryId> Entries = LoadNameBatch(Ar);
		Names.Reserve(Entries.Num());
		check(Entries.Num() == Numbers.Num());
		for (int64 Idx = 0, Num = Entries.Num(); Idx < Num; ++Idx)
		{
			Names.Emplace(Entries[Idx].ToName(Numbers[Idx]));
		}
	}

	~FBatchTextRoundtripper()
	{
		UnmountReadSchemas(BatchIds->GetBatchId());
	}

	void RoundtripTextForDiffTest(TConstArrayView<FInstance> Instances) const
	{
		PlainProps::UE::RoundtripText(*BatchIds, Objects, Instances, Format, {}, MarkupFormat);
	}

	void RoundtripTextForPackage(FStringView PackageName) const
	{
		PlainProps::UE::RoundtripText(*BatchIds, Objects, {}, Format, PackageName, MarkupFormat);
	}

	void PrintPackageToFile(FStringView PackageName) const
	{
		TUtf8StringBuilder<64> Markup;
		Markup.Reserve(2 << 10);
		switch (MarkupFormat)
		{
		case EMarkupFormat::Yaml: PrintYamlBatch(Markup, *BatchIds, Objects); break;
		case EMarkupFormat::Json: PrintJsonBatch(Markup, *BatchIds, Objects); break;
		}
		FUtf8StringView MarkupView = Markup.ToView();

		// Write to file
		const TCHAR* Extension = (MarkupFormat == EMarkupFormat::Yaml) ? TEXT(".yaml") : TEXT(".json");
		const FString Path = FPaths::Combine(FPaths::ProjectSavedDir(), TEXT("PlainProps"), PackageName) + Extension;
		if (TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileWriter(*Path)); Ar)
		{
			UE_LOGFMT(LogPlainPropsUObject, Log, "Writing {KB}KB {Format} as {Path}...", Markup.Len() >> 10, Extension + 1, *Path);
			Ar->Serialize((void*)MarkupView.GetData(), MarkupView.Len());
		}
	}

private:
	TArray<FStructView>			Objects;
	TArray<FSensitiveName>		Names;
	TUniquePtr<FBatchIds>		BatchIds;
	ESchemaFormat				Format;
	EMarkupFormat				MarkupFormat;
	
	template<typename T>
	static TConstArrayView<T> GrabNumAndArray(/* in-out */ FByteReader& It)
	{
		uint32 Num = It.Grab<uint32>();
		return MakeArrayView(reinterpret_cast<const T*>(It.GrabBytes(Num * sizeof(T))), Num);
	}
};

////////////////////////////////////////////////////////////////////////////////

struct FDiffDebug
{
	TArray<FInstance*> Instances;
	volatile const UTF8CHAR* Str;
};
static FDiffDebug GPpDiff, GTpsDiff, GUpsDiff;

static void DumpUObjectToJsonFile(const UObject* Object)
{
	EJsonStringifyFlags StringifyFlags = EJsonStringifyFlags::DisableDeltaEncoding | EJsonStringifyFlags::DisableNativeSerializers;
	if (FUtf8String Json(::UE::JsonObjectGraph::Stringify({Object}, {StringifyFlags})); Json.Len())
	{
		TStringBuilder<256> JsonPath;
		JsonPath << FPaths::Combine(FPaths::ProjectSavedDir(), TEXT("PlainProps"), Object->GetPackage()->GetName());
		JsonPath << "." << Object->GetName() + ".json";

		UE_LOGFMT(LogPlainPropsUObject, Log, "Writing {KB}KB json as {JsonPath}...", Json.Len() >> 10, *JsonPath);
		if (TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileWriter(JsonPath.ToString())); Ar)
		{
			Ar->Serialize((void*)Json.GetCharArray().GetData(), Json.GetCharArray().Num() - 1);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////

#if PP_WITH_ENGINE_HOOKS

struct FLinkerPropertyBatch
{
	explicit FLinkerPropertyBatch(const FLinkerSave& InLinkerSave)
		: LinkerSave(&InLinkerSave)
	{
	}

	explicit FLinkerPropertyBatch(FLinkerLoad& InLinkerLoad)
		: LinkerLoad(&InLinkerLoad)
	{
	}

	static FSoleMemberSpec Spec(FName*, FDeclId Id)
	{
		return {Id, GUE.Members.Id, Specify<uint64>()};
	}

	void Save(FMemberBuilder& Out, FName In, const FSaveContext&)
	{
		// see FLinkerSave::MapName
		int32 Index = LinkerSave->NameIndices.FindRef(In.GetDisplayIndex(), INDEX_NONE);
		int32 Number = In.GetNumber();
		uint64 Id;
		if (Index == INDEX_NONE)
		{
			UE_LOGFMT(LogPlainPropsUObject, Warning, "FName '{Name}' has not been harvested during save.", In);
			Id = 6666666; // temp: make these names easy to identify when looking at text output
		}
		else
		{
			Id = ((uint64(Number) << 32) | Index);
		}
		Out.Add(GUE.Members.Id, Id);
	}

	void Load(FName& Out, FStructLoadView In)
	{
		uint64 Id;
		LoadSole<uint64>(&Id, In);
		if (Id != 6666666) // todo: 6666666 is prototype code
		{
			int32 Index = Id & 0xFFFFFFFF;
			int32 Number = Id >> 32;
			FNameEntryId Entry = LinkerLoad->NameMap[Index];
			FName Name = FName::CreateFromDisplayId(Entry, Number);
			Out = Name;
		}
		else
		{
			Out = FName();
		}
	}

	static FSoleMemberSpec Spec(FText*, FDeclId Id)
	{
		return {Id, GUE.Members.Id, Specify<int32>()};
	}

	void Save(FMemberBuilder& Out, const FText& In, const FSaveContext&)
	{
		Out.Add(GUE.Members.Id, Texts.Num());
		Texts.Add(In);
	}

	void Load(FText& Out, FStructLoadView In) const
	{
		Out = Texts[LoadSole<int32>(In)];
	}

	static FSoleMemberSpec Spec(FObjectHandle*, FDeclId Id)
	{
		return {Id, GUE.Members.Id, Specify<int32>()};
	}

	void Save(FMemberBuilder& Out, const FObjectHandle& In, const FSaveContext&)
	{
		// see FLinkerSave::MapObject
		UObject* Object = FObjectPtr(In).Get();
		FPackageIndex Index = LinkerSave->ObjectIndicesMap.FindRef(Object);
		int32 Idx = *reinterpret_cast<int32*>(&Index);
		if (Object && Index.IsNull())
		{
			if (!LinkerSave->IsUnsaveable(Object))
			{
				UE_LOGFMT(LogPlainPropsUObject, Warning, "UObject '{Object}' reference has not been harvested during save.", Object->GetFullName());
				Idx = 9999999; // temp: make these objects easy to identify when looking at text output
			}
		}
		Out.Add(GUE.Members.Id, Idx);
	}

	void Load(FObjectHandle& Out, FStructLoadView In)
	{
		int32 Idx;
		LoadSole<int32>(&Idx, In);
		Idx = (Idx == 9999999) ? 0 : Idx; // todo: 9999999 is prototype code
		FPackageIndex PackageIdx = *reinterpret_cast<FPackageIndex*>(&Idx);
		if (PackageIdx.IsImport())
		{
			UObject* Object = LinkerLoad->ImportMap[PackageIdx.ToImport()].XObject;
			Object = Object ? Object : LinkerLoad->ResolveResource(PackageIdx);
			Out = ::UE::CoreUObject::Private::MakeObjectHandle(Object);
		}
		else if (PackageIdx.IsExport())
		{
			UObject* Object = LinkerLoad->ExportMap[PackageIdx.ToExport()].Object;
			Object = Object ? Object : LinkerLoad->ResolveResource(PackageIdx);
			Out = ::UE::CoreUObject::Private::MakeObjectHandle(Object);
		}
		else
		{
			check(PackageIdx.IsNull());
			Out = FObjectHandle();
		}
	}

	static FSoleMemberSpec Spec(FWeakObjectPtr*, FDeclId Id)
	{
		return {Id, GUE.Members.Id, Specify<int32>()};
	}

	void Save(FMemberBuilder& Out, const FWeakObjectPtr& In, const FSaveContext& Ctx)
	{
		UObject* Object = In.Get(true);
		Save(Out, ::UE::CoreUObject::Private::MakeObjectHandle(Object), Ctx);
	}

	void Load(FWeakObjectPtr& Out, FStructLoadView In)
	{
		FObjectHandle Handle;
		Load(Handle, In);
		Out = ::UE::CoreUObject::Private::ResolveObjectHandle(Handle);
	}

	static FSoleMemberSpec Spec(FSoftObjectPath*, FDeclId Id)
	{
		return {Id, GUE.Members.Id, Specify<int32>()};
	}

	void Save(FMemberBuilder& Out, const FSoftObjectPath& In, const FSaveContext& Ctx)
	{
		// see FLinkerSave::MapSoftObjectPath
		int32 Idx = LinkerSave->SoftObjectPathIndices.FindRef(In, INDEX_NONE);
		if (Idx == INDEX_NONE)
		{
			UE_LOGFMT(LogPlainPropsUObject, Warning, "SoftObjectPath {Path} is not mapped.", In);
		}
		Out.Add(GUE.Members.Id, Idx);
	}

	void Load(FSoftObjectPath& Out, FStructLoadView In)
	{
		int32 Idx;
		LoadSole<int32>(&Idx, In);
		if (Idx != INDEX_NONE)
		{
			check(Idx < LinkerLoad->SoftObjectPathList.Num());
			Out = LinkerLoad->SoftObjectPathList[Idx];
		}
		else
		{
			Out.Reset();
		}
	}

	static FSoleMemberSpec Spec(FSoftObjectPtr*, FDeclId Id)
	{
		return Spec((FSoftObjectPath*)nullptr, Id);
	}

	void Save(FMemberBuilder& Out, const FSoftObjectPtr& In, const FSaveContext& Ctx)
	{
		return Save(Out, In.ToSoftObjectPath(), Ctx);
	}

	void Load(FSoftObjectPtr& Out, FStructLoadView In)
	{
		FSoftObjectPath Temp;
		Load(Temp, In);
		Out = Temp;
	}

	static FSoleMemberSpec Spec(FLazyObjectPtr*, FDeclId Id)
	{
		return {Id, GUE.Members.Id, FMemberSpec(GUE.Structs.Guid)};
	}
	
	static void Save(FMemberBuilder& Out, const FLazyObjectPtr& In, const FSaveContext& Ctx)
	{
		FBuiltStruct* Guid = SaveStruct(&In.GetUniqueID(), GUE.Structs.Guid, Ctx);
		Out.AddStruct(GUE.Members.Id, GUE.Structs.Guid, Guid);
	}

	static void Load(FLazyObjectPtr& Out, FStructLoadView In)
	{
		Out.ResetWeakPtr();
		LoadSoleStruct(&Out.GetUniqueID(), In);
	}

	TArray<FText>&	GrabTexts()							{ return Texts; }
	void			GiveTexts(TArray<FText>&& InTexts)	{ Texts = MoveTemp(InTexts); }

private:
	union
	{
		const FLinkerSave* LinkerSave;
		FLinkerLoad* LinkerLoad;
		const FLinker* Linker;
	};

	TArray<FText> Texts; // Tricky to serialize non-intrusively
};

struct FLinkerPropertyBindings : TCustomPropertyBindings<FLinkerPropertyBatch>
{
	FLinkerPropertyBindings(FLinkerPropertyBatch& Batch, const FCustomBindings& Underlay)
	: TCustomPropertyBindings<FLinkerPropertyBatch>(Batch, Underlay)
	, SoftObjectPath(Batch)
	{
		Bind(GUE.Structs.SoftObjectPath, SoftObjectPath);
	}
	
	TCustomPropertyBinding<FSoftObjectPath, FLinkerPropertyBatch> SoftObjectPath;
};

static UPackage* PinPackage(UPackage* Package)
{
	if (Package)
	{
		Package->AddRef();
		ForEachObjectWithPackage(Package, [](UObject* Obj)
		{
			Obj->AddRef();
			return true;
		});
	}
	return Package;
}

static UPackage* UnpinPackage(UPackage* Package)
{
	if (Package)
	{
		Package->ReleaseRef();
		ForEachObjectWithPackage(Package, [](UObject* Obj)
		{
			Obj->ReleaseRef();
			return true;
		});
	}
	return Package;
}

static FString GetPackageSaveFilename(UPackage* Package)
{
	const FString& Extension = Package->ContainsMap() ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension();
	FString SaveFilename = FPaths::Combine(*FPaths::ProjectSavedDir(), TEXT("PlainProps"), Package->GetName() + Extension);
	return SaveFilename;
}

struct FPlainPropsLinkerSandbox
{
	static constexpr uint32 LoadFlags = (LOAD_Async | LOAD_NoVerify | LOAD_SkipLoadImportedPackages);
	static constexpr uint32 BaseNumber = 8848;
	static constexpr uint32 PPNumber = 8849;

	struct FSavedPackage
	{
		FName PackageName;
		FPackagePath PackagePath;
		FSharedBuffer PackageData;
		TArray64<uint8> PlainPropsData;
		TArray<FText> Texts;
	};

	struct FMountedLinker
	{
		FMountedLinker(const FSavedPackage& Saved, FName InPackageName)
			: PackageName(InPackageName)
			, PackagePath(Saved.PackagePath)
			, Package(NewObject<UPackage>(/*Outer*/nullptr, PackageName))
			, LinkerReader(new FLargeMemoryReader(static_cast<const uint8*>(Saved.PackageData.GetData()), Saved.PackageData.GetSize(), ELargeMemoryReaderFlags::Persistent, PackageName))
			, LinkerLoad(FLinkerLoad::CreateLinker(FUObjectThreadContext::Get().GetSerializeContext(), Package, Saved.PackagePath, LoadFlags, LinkerReader))
		{
			check(LinkerLoad->GetLoader_Unsafe() == LinkerReader);
		}

		~FMountedLinker()
		{
			// This will first destroy and free the LinkerReader then the LinkerLoad
			ResetLoaders(Package);
		}

		FName PackageName;
		FPackagePath PackagePath;
		UPackage* Package;
		FLargeMemoryReader* LinkerReader;
		FLinkerLoad* LinkerLoad;
	};

	struct FMountedLinkerBatch
	{
		FMountedLinkerBatch(const FSavedPackage& Saved, FName InPackageName)
			: MountedLinker(Saved, InPackageName)
			, Properties(*MountedLinker.LinkerLoad)
			, Customs(Properties, GUE.Customs)
			, BatchView(MakeMemoryView(Saved.PlainPropsData))
			, RemapPackageName(FSensitiveName(Saved.PackageName), FSensitiveName(InPackageName))
			, BatchLoader(Customs.Overlay, BatchView, RemapPackageName)
		{
			Properties.GiveTexts(CopyTemp(Saved.Texts));
		}

		FMountedLinker MountedLinker;
		FLinkerPropertyBatch Properties;
		FLinkerPropertyBindings Customs;
		FMemoryView BatchView;
		TPair<FSensitiveName, FSensitiveName> RemapPackageName;
		FPersistentBatchLoader BatchLoader;
	};

	class FPackageStoreBackend final : public IPackageStoreBackend
	{
		struct FEntry
		{
			FName PackageName;
			EPackageExtension Extension;
		};
		TMap<FName, FEntry> Packages;

	public:
		void AddPackage(FName PackageName, const FPackagePath& PackagePath)
		{
			Packages.Add(PackagePath.GetPackageFName(), { PackageName, PackagePath.GetHeaderExtension() });
		}

		void ClearPackages()
		{
			Packages.Reset();
		}

		virtual EPackageLoader GetSupportedLoaders() override
		{
			return EPackageLoader::LinkerLoad;
		}

		virtual void OnMounted(TSharedRef<const FPackageStoreBackendContext>) override {}
		virtual void BeginRead() override {}
		virtual void EndRead() override {}

		virtual EPackageStoreEntryStatus GetPackageStoreEntry(FPackageId PackageId, FName PackageName, FPackageStoreEntry& OutEntry) override
		{
			if (const FEntry* Entry = Packages.Find(PackageName))
			{
				OutEntry.LoaderType = EPackageLoader::LinkerLoad;
				OutEntry.PackageExtension = Entry->Extension;
#if WITH_EDITORONLY_DATA
				OutEntry.LinkerLoadCaseCorrectedPackageName = Entry->PackageName;
#endif
				return EPackageStoreEntryStatus::Ok;
			}
			return EPackageStoreEntryStatus::Missing;
		}

		virtual bool GetPackageRedirectInfo(FPackageId, FName&, FPackageId&) override
		{
			return false;
		}
	};

	static TMap<FName, FSavedPackage> SavedPackages;
	static TMap<FName, TUniquePtr<FMountedLinker>> MountedBaselinePackages;
	static TMap<FName, TUniquePtr<FMountedLinkerBatch>> MountedPlainpropsPackages;
	static TArray<const UClass*> PendingClasses;
	static TSharedPtr<FPackageStoreBackend> PackageStoreBackend;

#if WITH_EDITOR
	static void OnBlueprintCDOCompiled(UObject* InObject, const FObjectPostCDOCompiledContext&)
	{
		const UClass* Class = InObject->GetClass();
		PendingClasses.AddUnique(Class);
	}
#endif

	static void SavePlainProps(const FLinkerSave& LinkerSave)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SavePlainProps);
		FName PackageName = LinkerSave.LinkerRoot->GetFName();

		UE_LOGFMT(LogPlainPropsUObject, Log, " Saving PlainProps for {PackageName}...", PackageName);
		FLinkerPropertyBatch Properties(LinkerSave);
		FLinkerPropertyBindings Customs(Properties, GUE.Customs);
		FBatchSaver BatchSaver(Customs.Overlay, LinkerSave.ExportMap.Num());
		for (const FObjectExport& Export : LinkerSave.ExportMap)
		{
			const UStruct* Struct;
#if PP_OVERRIDE_OBJECT_LOADING
			if ((Struct = Cast<UStruct>(Export.Object)))
			{
				// Save data defined UStructs as meta structs (_C_META) with no archetype
				check(!FOverridableManager::Get().GetOverriddenProperties(Export.Object));
				FBindId Id = GUE.Names.IndexBindId(PlainProps::UE::IndexStructMeta(Struct));
				if (GUE.Customs.FindStruct(Id))
				{
					BatchSaver.SaveMeta(Id, Struct);
				}
				else
				{
					BatchSaver.Skip(Id, Export.Object);
				}
			}
			else
#endif
			{
				Struct = SkipEmptyBases(Export.Object->GetClass());
				FBindId Id = GUE.Names.IndexBindId(PlainProps::UE::IndexStruct(Struct));
				if (const FOverriddenPropertySet* Overriddes = FOverridableManager::Get().GetOverriddenProperties(Export.Object))
				{
					BatchSaver.SaveObjectWithOverrides(Id, Export.Object, *Overriddes, Export.Object->GetArchetype());
				}
				else if (GUE.Schemas.FindStruct(Id))
				{
					BatchSaver.Save(Id, Export.Object, Export.Object->GetArchetype());
				}
				else
				{
					BatchSaver.Skip(Id, Export.Object);
				}
			}
		}
		TArray64<uint8> StableData = BatchSaver.Write();

		FSavedPackage& SavedPackage = SavedPackages.FindOrAdd(PackageName);
		SavedPackage.PackageName = PackageName;
		SavedPackage.PackagePath = !LinkerSave.LinkerRoot->GetLoadedPath().IsEmpty() ?
			LinkerSave.LinkerRoot->GetLoadedPath() :
			FPackagePath::FromLocalPath(GetPackageSaveFilename(LinkerSave.LinkerRoot));
		SavedPackage.PlainPropsData = MoveTemp(StableData);
		SavedPackage.Texts = Properties.GrabTexts();
	}

	static void SavePackageToMemory(const FLinkerSave& LinkerSave, uint8* Data, int64 Size)
	{
		FName PackageName = LinkerSave.LinkerRoot->GetFName();
		FSavedPackage& SavedPackage = SavedPackages.FindChecked(PackageName);
		SavedPackage.PackageData = FSharedBuffer::TakeOwnership(Data, Size, FMemory::Free);
	}

	static bool OpenPackageReader(const FPackagePath& PackagePath, FArchive*& Ar)
	{
		FName PackageName = PackagePath.GetPackageFName();
		if (FSavedPackage* SavedData = SavedPackages.Find(PackageName))
		{
			Ar = new FLargeMemoryReader(static_cast<const uint8*>(SavedData->PackageData.GetData()), SavedData->PackageData.GetSize(), ELargeMemoryReaderFlags::Persistent, PackageName);
			return true;
		}
		return false;
	}

	[[nodiscard]] static UPackage* MountBaselinePackage(FName PackageName, bool bClone)
	{
		if (FSavedPackage* SavedData = SavedPackages.Find(PackageName))
		{
			FName NewPackageName = PackageName;
			if (bClone)
			{
				NewPackageName = *WriteToString<256>("/CloneBaseline", PackageName);
			}
			UE_LOGFMT(LogPlainPropsUObject, Log, " Mounting baseline for {Package}...", NewPackageName);
			TUniquePtr<FMountedLinker> MountedInstance(new FMountedLinker(*SavedData, NewPackageName));
			UPackage* Out = MountedInstance->Package;
			MountedBaselinePackages.Emplace(NewPackageName, MoveTemp(MountedInstance));
			PackageStoreBackend->AddPackage(NewPackageName, SavedData->PackagePath);
			return Out;
		}
		return nullptr;
	}

	[[nodiscard]] static UPackage* MountPlainPropsPackage(FName PackageName, bool bClone)
	{
		if (FSavedPackage* SavedData = SavedPackages.Find(PackageName))
		{
			FName NewPackageName = PackageName;
			if (bClone)
			{
				NewPackageName = *WriteToString<256>("/ClonePlainProps", PackageName);
			}
			UE_LOGFMT(LogPlainPropsUObject, Log, " Mounting plainprops for {Package}...", NewPackageName);
			TUniquePtr<FMountedLinkerBatch> MountedInstance(new FMountedLinkerBatch(*SavedData, NewPackageName));
			UPackage* Out = MountedInstance->MountedLinker.Package;
			MountedPlainpropsPackages.Add(NewPackageName, MoveTemp(MountedInstance));
			PackageStoreBackend->AddPackage(NewPackageName, SavedData->PackagePath);
			return Out;
		}
		return nullptr;
	}

	static void Initialize(uint32 NumPackages)
	{
		SavedPackages.Reserve(NumPackages);
		MountedBaselinePackages.Reserve(NumPackages);
		MountedPlainpropsPackages.Reserve(NumPackages);
		
		PackageStoreBackend = MakeShared<FPackageStoreBackend>();
		FPackageStore::Get().Mount(PackageStoreBackend.ToSharedRef(), /*Priority=*/10);
	}

	static void Shutdown()
	{
		MountedBaselinePackages.Empty();
		MountedPlainpropsPackages.Empty();
		SavedPackages.Empty();
		PackageStoreBackend->ClearPackages();
	}

	static UPackage* LoadBaselinePackage(UPackage* Package)
	{
		if (TUniquePtr<FMountedLinker>* MountedLinkerPtr = MountedBaselinePackages.Find(Package->GetFName()))
		{
			FMountedLinker& MountedLinker = **MountedLinkerPtr;

			UE_LOGFMT(LogPlainPropsUObject, Log, " Loading baseline for {Package}...", MountedLinker.PackageName);
			return LoadPackage(MountedLinker.Package, MountedLinker.PackagePath, LoadFlags);
		}
		return nullptr;
	}

	static UPackage* LoadPlainPropsPackage(UPackage* Package)
	{
		if (TUniquePtr<FMountedLinkerBatch>* LinkerBatchPtr = MountedPlainpropsPackages.Find(Package->GetFName()))
		{
			FMountedLinkerBatch& LinkerBatch = **LinkerBatchPtr;

			UE_LOGFMT(LogPlainPropsUObject, Log, " Loading PlainProps for {Package}...", LinkerBatch.MountedLinker.PackageName);
			return LoadPackage(LinkerBatch.MountedLinker.Package, LinkerBatch.MountedLinker.PackagePath, LoadFlags);
		}

		return nullptr;
	}

	static void RoundtripToTextFromMemory(FName PackageName, EMarkupFormat MarkupFormat)
	{
		if (FSavedPackage* SavedData = SavedPackages.Find(PackageName))
		{
			FBatchTextRoundtripper StableBatch(MakeMemoryView(SavedData->PlainPropsData), ESchemaFormat::StableNames, MarkupFormat);
			StableBatch.RoundtripTextForPackage(PackageName.ToString());
		}
	}

	static void PrintPackageToFile(FName PackageName, EMarkupFormat MarkupFormat)
	{
		if (FSavedPackage* SavedData = SavedPackages.Find(PackageName))
		{
			FBatchTextRoundtripper StableBatch(MakeMemoryView(SavedData->PlainPropsData), ESchemaFormat::StableNames, MarkupFormat);
			StableBatch.PrintPackageToFile(PackageName.ToString());
		}
	}

	static bool LoadObject(UObject* Object, FArchive& Ar)
	{
		if (!Ar.IsLoading() || !Ar.IsPersistent() || Ar.HasAnyPortFlags(PPF_Duplicate|PPF_DuplicateForPIE))
		{
			return false;
		}

		FName PackageName = Object->GetPackage()->GetFName();
		TUniquePtr<FMountedLinkerBatch>* LinkerBatchPtr = MountedPlainpropsPackages.Find(PackageName);
		check(LinkerBatchPtr);
		FMountedLinkerBatch& LinkerBatch = **LinkerBatchPtr;

		FPersistentBatchLoader& BatchLoader = LinkerBatch.BatchLoader;
		FLinkerLoad* LinkerLoad = LinkerBatch.MountedLinker.LinkerLoad;
		check(LinkerLoad == Object->GetLinker());

		FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();
		check(Object == SerializeContext->SerializedObject);

		const int32 ExportIndex = Object->GetLinkerIndex();
		const FObjectExport& Export = LinkerLoad->ExportMap[ExportIndex];
		check(Export.Object == Object);

		if (PendingClasses.Num())
		{
			// Re-bind blueprint classes whenever their CDO has been recompiled
			for (const UClass* Class : PendingClasses)
			{
				FBindId Id = GUE.Names.IndexBindId(PlainProps::UE::IndexStruct(Class));
				if (GUE.Schemas.FindStruct(Id))
				{
					UE_LOGFMT(LogPlainPropsUObject, Log, " Re-binding compiled blueprint class {BindId}", GUE.Debug.Print(Id));
					GUE.Schemas.DropStruct(Id);
					GUE.Metadatas.DropMetadata(Id);
				}
				else
				{
					UE_LOGFMT(LogPlainPropsUObject, Log, " Binding compiled blueprint class {BindId}", GUE.Debug.Print(Id));
				}
				UE::BindStruct(Id, Class);
			}
			PendingClasses.Reset();
		}
		// Bind dynamic classes at the time we load their CDO.
		// This is first time the struct binding is needed,
		// and we also know for sure that the UClass has already been linked.
		if (Object->HasAllFlags(RF_ClassDefaultObject))
		{
			UClass* Class = Object->GetClass();
			if (!Class->HasAllClassFlags(CLASS_Native))
			{
				FBindId Id = GUE.Names.IndexBindId(PlainProps::UE::IndexStruct(Class));
				UE_LOGFMT(LogPlainPropsUObject, Log, " Binding loaded dynamic class {BindId} for loading CDO {CDO}",
					GUE.Debug.Print(Id), *Object->GetPathName());
				check(!GUE.Schemas.FindStruct(Id))
				UE::BindStruct(Id, Class);
			}
		}
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(LoadPlainProps);

			if (FOverriddenPropertySet* Overrides = FOverridableManager::Get().GetOverriddenProperties(Object))
			{
				FScopeRestoreOverrides RestoreScope(Overrides);

				BatchLoader.LoadObject(Object, ExportIndex);

				BatchLoader.RestoreObjectOverrides(Object, ExportIndex, *Overrides);
			}
			else
			{
				BatchLoader.LoadObject(Object, ExportIndex);
			}
		}
		// We only need to adjust the archive when we override property loading,
		// and not when completely overriding custom serialize object loading.
		if (GSerializePropertiesOverride && !GSerializeObjectOverride)
		{
#if WITH_EDITOR
			const int32 Pos = Ar.Tell();
			check(Pos == SerializeContext->SerializedObjectScriptStartOffset);
			const int32 ScriptSerializationSize = Export.ScriptSerializationEndOffset - Export.ScriptSerializationStartOffset;
			check(ScriptSerializationSize >= 0);
			Ar.Seek(Pos + ScriptSerializationSize);
#else
			// todo: SerializedObjectScriptStartOffset does not exist in client builds
			unimplemented();
#endif
		}
		return true;
	}

	static bool LoadProperties(FStructuredArchive::FSlot& Slot, uint8* Data)
	{
		FArchive& Ar = Slot.GetUnderlyingArchive();
		if (!LoadObject((UObject*)Data, Ar))
		{
			return false;
		}

		return true;
	}
};

TMap<FName, FPlainPropsLinkerSandbox::FSavedPackage> FPlainPropsLinkerSandbox::SavedPackages;
TMap<FName, TUniquePtr<FPlainPropsLinkerSandbox::FMountedLinkerBatch>> FPlainPropsLinkerSandbox::MountedPlainpropsPackages;
TMap<FName, TUniquePtr<FPlainPropsLinkerSandbox::FMountedLinker>> FPlainPropsLinkerSandbox::MountedBaselinePackages;
TArray<const UClass*> FPlainPropsLinkerSandbox::PendingClasses;
TSharedPtr<FPlainPropsLinkerSandbox::FPackageStoreBackend> FPlainPropsLinkerSandbox::PackageStoreBackend;

#endif

////////////////////////////////////////////////////////////////////////////////

static FString GetSavePackageFilename(UPackage* Package)
{
	const FString& Extension = Package->ContainsMap() ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension();
	return FPaths::Combine(*FPaths::ProjectSavedDir(), TEXT("PlainProps"), Package->GetName() + Extension);
}

static bool IsNativeCDO(UObject* Object)
{
	return Object->HasAllFlags(RF_ClassDefaultObject) && Object->GetClass()->HasAllClassFlags(CLASS_Native);
}

using FDiffSet = TSet<FDiffPath>;

FDiffSet Flatten(const FDiffTree& DiffTree)
{
	FDiffSet DiffSet;

	FDiffPath CurrentPath;
	CurrentPath.Reserve(DiffTree.Num());

	for (int32 N = DiffTree.Num() - 1; N >= 0; --N)
	{
		FDiffNode Node = DiffTree[N];

		int32 PrecedingSibling = Node.PrecedingSibling;

		// The preceding sibling index has no meaning when we convert into a path.
		Node.PrecedingSibling = INDEX_NONE;
		CurrentPath.Add(Node);

		bool bHasPrecedingSibling = (PrecedingSibling != INDEX_NONE);

		// Terminate the path if we have finished a 'run'.
		if ((N == 0) || bHasPrecedingSibling)
		{
			DiffSet.Add(CurrentPath);

			if (bHasPrecedingSibling)
			{
				// The previous sibling shares the same parent path as this node.
				CurrentPath.Pop();
			}
		}
	}

	return DiffSet;
}

int32 RoundtripViaPlainBatch(TConstArrayView<UObject*> Objects, ERoundtrip Options)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RoundtripViaPlainBatch);

	TArray<FInstance> Instances;
	if (Objects.Num())
	{
		UE_LOGFMT(LogPlainPropsUObject, Display, "Gathering UObjects in Assets...");
		for (UObject* Object : Objects)
		{
			if (Object->IsA<UPackage>())
			{
				ForEachObjectWithPackage(Object->GetPackage(), [&Instances](UObject* Obj)
				{
					UClass* Class = Obj->GetClass();
					if (ShouldBind(Class) && IncludeClass(Class))
					{
						Instances.Emplace(Obj->GetPathName(), Obj);
					}
					return true;
				});
			}
			else
			{
				Instances.Emplace(Object->GetPathName(), Object);
			}
		}
	}
	else
	{
		UE_LOGFMT(LogPlainPropsUObject, Display, "Gathering all non-empty UObjects...");
		static constexpr EObjectFlags SkipFlags =  RF_MirroredGarbage | RF_InheritableComponentTemplate;
		for (TObjectIterator<UObject> It(SkipFlags); It; ++It)
		{
			UObject* Object = *It;
			UClass* Class = Object->GetClass();
			if (ShouldBind(Class) && IncludeClass(Class) && !IsNativeCDO(Object))
			{
				Instances.Emplace(Object->GetPathName(), Object);
			}
		}
	}

	UE_LOGFMT(LogPlainPropsUObject, Display, "Sorting {Num} UObjects ...", Instances.Num());
	Algo::Sort(Instances, [](const FInstance& A, const FInstance& B)
	{
		return FPlatformString::Strcmp(*A.PathName, *B.PathName) < 0;
	});

	// Create CDOs if needed and then clones for PP and TPS tests
	UE_LOGFMT(LogPlainPropsUObject, Display, "Cloning {Num} UObjects up to 4 times...", Instances.Num());
	for (FInstance& Instance : Instances)
	{
		Instance.Init();
	}
	FlushAsyncLoading();

	for (FInstance& Instance : Instances)
	{
		uint32 N = 1 + &Instance - Instances.GetData();
		Instance.Base = Instance.Arch ? MakeEmptyInstance(Instance.Orig, FName("Base", N)) : nullptr;
		if (EnumHasAnyFlags(Options, ERoundtrip::PP))
		{
			Instance.PP = MakeEmptyInstance(Instance.Orig, FName("PP", N));
		}
		if (EnumHasAnyFlags(Options, ERoundtrip::TPS))
		{
			Instance.TPS = MakeEmptyInstance(Instance.Orig, FName("TPS", N));
		}
		if (EnumHasAnyFlags(Options, ERoundtrip::UPS))
		{
			Instance.UPS = MakeEmptyInstance(Instance.Orig, FName("UPS", N));
		}

		check(!Instance.PP  || (Instance.PP  != Instance.Base));
		check(!Instance.TPS || (Instance.TPS != Instance.Base));
		check(!Instance.UPS || (Instance.UPS != Instance.Base));
	}

	FMemoryBatch Plain;
	TCustomPropertyBindings<FMemoryPropertyBatch> Customs(Plain.Properties, GUE.Customs);

	// We want to ignore diffs that happen during TPS serialization when comparing the
	// results of PlainProps serialization. For this reason we start with TPS. Such
	// differences could be a result of the way objects are instantiated in the test.
	FArchivedProperties Tps;
	if (EnumHasAnyFlags(Options, ERoundtrip::TPS))
	{
		UE_LOGFMT(LogPlainPropsUObject, Display, "Saving UObjects to TPS archive...");
		FPropertyWriter Archive(/* out */ Tps);
		SaveArchive<false>(Archive, Instances);
	}

	if (EnumHasAnyFlags(Options, ERoundtrip::TPS))
	{
		UE_LOGFMT(LogPlainPropsUObject, Display, "Loading UObjects from TPS archive...");
		FPropertyReader Archive(Tps);
		LoadArchive<false>(Archive, Instances);
	}

	static const FName SkipClasses[] = {
	"BodySetup", // Skips some structs due to native FCollisionResponse::operator==
	"NiagaraScript", "NiagaraNodeFunctionCall", "NiagaraMeshRendererProperties", // FNiagaraTypeDefinition::Serialize resets ClassStructOrEnum
	};

	const EDiffGather GatherMode = EDiffGather::All;

	// Map: OriginalInstance -> Diff Paths, that we use as an ignore list.
	// When a diff happens during the original vs TPS comparison we want to
	// exclude it during the original vs PlainProps comparison.
	TMap<UObject*, FDiffSet> IgnoreDiffMap;

	if (EnumHasAnyFlags(Options, ERoundtrip::TPS))
	{
		// Diff original vs TPS
		UE_LOGFMT(LogPlainPropsUObject, Display, "Diffing UObjects roundtripped via TPS...");
		FDiffContext DiffCtx = { {GUE.Schemas, Customs.Overlay }, GatherMode };
		TUtf8StringBuilder<256> TpsDiffs;
		TArray<int32> TpsDiffIdxs;
		for (FInstance& Instance : Instances)
		{
			if (Algo::Find(SkipClasses, Instance.Orig->GetClass()->GetFName()))
			{
				continue;
			}

			if (DiffStructs(Instance.Orig, Instance.TPS, Instance.Id, /* in-out */ DiffCtx))
			{
				check(!DiffCtx.Diffs.IsEmpty());

				FDiffSet DiffSet = Flatten(DiffCtx.Diffs);
				check((GatherMode == EDiffGather::All) || (DiffSet.Num() == 1));

				GTpsDiff.Instances.Add(&Instance);
				PrintDiffs(/* out */ TpsDiffs, GUE.Names, DiffSet.Array());

				IgnoreDiffMap.Add(Instance.Orig, MoveTemp(DiffSet));

				DiffCtx.Reset();
				TpsDiffs.Append(" in ");
				TpsDiffs.Append(Instance.Orig->GetFullName());
				TpsDiffs.Append("\n");
				TpsDiffIdxs.Add(&Instance - &Instances[0]);
				
				//FArchivedProperties Tmp;
				//FPropertyWriter(/* out */ Tmp).WriteProperties(Instance.Orig, Instance.Base);
				//FPropertyReader(Tmp).ReadProperties(Instance.UPS, Instance.Base);

				//FDiffContext TmpCtx = DiffCtx;
				//bool bOU = DiffStructs(Instance.Id, Instance.Orig, Instance.UPS, /* in-out */ TmpCtx);
				//bool bOT = DiffStructs(Instance.Id, Instance.Orig, Instance.TPS, /* in-out */ TmpCtx);
				//bool bOP = DiffStructs(Instance.Id, Instance.Orig, Instance.PP, /* in-out */ TmpCtx);
				//TUtf8StringBuilder<256> TmpDiff;
				//PrintDiff(/* out */ TmpDiff, TmpCtx.Out, GUE.Names);
			}
		}
		GTpsDiff.Str = TpsDiffs.GetData();
		UE_LOGFMT(LogPlainPropsUObject, Display, "Detected {Diffs} diffs in {Objs} UObjects saved in a {KB}KB value stream using TPS", TpsDiffIdxs.Num(), Instances.Num(), Tps.Data.NumBytes() / 1024);
	}

	// Save
	UE_LOGFMT(LogPlainPropsUObject, Display, "Saving UObjects to PlainProps with InMemoryNames...");
	{
		FBatchSaver Batch(Customs.Overlay, GUObjectArray.GetObjectArrayNum());
		SavePlainProps(Batch, Instances);
		Plain.Data = Batch.Write(/* out */ &Plain.RuntimeIds);

		if (EnumHasAnyFlags(Options, ERoundtrip::TextMemory))
		{
			EMarkupFormat MarkupFormat = EnumHasAnyFlags(Options, ERoundtrip::JSON) ? EMarkupFormat::Json : EMarkupFormat::Yaml;
			FBatchTextRoundtripper MemoryBatch(MakeMemoryView(Plain.Data), ESchemaFormat::InMemoryNames, MarkupFormat);
			MemoryBatch.RoundtripTextForDiffTest(Instances);
		}
		if (EnumHasAnyFlags(Options, ERoundtrip::TextStable))
		{
			UE_LOGFMT(LogPlainPropsUObject, Display, "Saving UObjects to PlainProps with StableNames...");
			TArray64<uint8> StableData = Batch.Write();

			EMarkupFormat MarkupFormat = EnumHasAnyFlags(Options, ERoundtrip::JSON) ? EMarkupFormat::Json : EMarkupFormat::Yaml;
			FBatchTextRoundtripper StableBatch(MakeMemoryView(StableData), ESchemaFormat::StableNames, MarkupFormat);
			StableBatch.RoundtripTextForDiffTest(Instances);
		}
	}

	// Load
	uint32 NumPpDiffs = 0;
	if (EnumHasAnyFlags(Options, ERoundtrip::PP))
	{
		UE_LOGFMT(LogPlainPropsUObject, Display, "Loading UObjects from PlainProps...");
		FMemoryBatchLoader Batch(Customs.Overlay, MakeMemoryView(Plain.Data), Plain.RuntimeIds);
		LoadPlainProps(Batch, Instances);

		// Diff original vs PlainProps
		UE_LOGFMT(LogPlainPropsUObject, Display, "Diffing UObjects roundtripped via PlainProps...");
		FDiffContext DiffCtx = { {GUE.Schemas, Customs.Overlay }, GatherMode };
		TUtf8StringBuilder<256> PpDiffPaths;

		for (FInstance& Instance : Instances)
		{
			if (DiffStructs(Instance.Orig, Instance.PP, Instance.Id, /* in-out */ DiffCtx))
			{
				check(!DiffCtx.Diffs.IsEmpty());

				FDiffSet DiffSet = Flatten(DiffCtx.Diffs);
				check((GatherMode == EDiffGather::All) || (DiffSet.Num() == 1));

				FDiffSet FiltertedDiffs;
				const FDiffSet* IgnoreDiffs = IgnoreDiffMap.Find(Instance.Orig);
				if (IgnoreDiffs)
				{
					FiltertedDiffs = DiffSet.Difference(*IgnoreDiffs);
				}

				FDiffSet& Diffs = IgnoreDiffs ? FiltertedDiffs : DiffSet;

				if (!Diffs.IsEmpty())
				{
					PpDiffPaths.Append("\n");
					GPpDiff.Instances.Add(&Instance);
					PrintDiffs(/* out */ PpDiffPaths, GUE.Names, Diffs.Array());

					//Batch.Reload(Instance.UPS, &Instance - Instances.GetData());
					PpDiffPaths.Append(" in ");
					PpDiffPaths.Append(Instance.Orig->GetFullName());
					++NumPpDiffs;
				}

				DiffCtx.Reset();
			}
		}
		GPpDiff.Str = PpDiffPaths.GetData();
		UE_LOGFMT(LogPlainPropsUObject, Display, "Detected {Diffs} diffs in {Objs} UObjects saved in a {KB}KB value stream using PlainProps{DiffText}", NumPpDiffs, Instances.Num(), Plain.Data.NumBytes() / 1024, PpDiffPaths.ToString());
		for (const FInstance* Instance : GPpDiff.Instances)
		{
			DumpUObjectToJsonFile(Instance->Orig);
			DumpUObjectToJsonFile(Instance->PP);
		}
	}

	if (EnumHasAnyFlags(Options, ERoundtrip::UPS))
	{
		FArchivedProperties Ups;
		UE_LOGFMT(LogPlainPropsUObject, Display, "Saving UObjects to UPS archive...");
		{
			FPropertyWriter Archive(/* out */ Ups);
			SaveArchive<true>(Archive, Instances);
		}

		UE_LOGFMT(LogPlainPropsUObject, Display, "Loading UObjects from UPS archive...");
		{
			FPropertyReader Archive(Ups);
			LoadArchive<true>(Archive, Instances);
		}

		// Diff original vs UPS
		UE_LOGFMT(LogPlainPropsUObject, Display, "Diffing UObjects roundtripped via UPS...");
		FDiffContext DiffCtx = { {GUE.Schemas, Customs.Overlay }, GatherMode };
		TUtf8StringBuilder<256> UpsDiffs;
		TArray<int32> UpsDiffIdxs;
		for (FInstance& Instance : Instances)
		{
			if (Algo::Find(SkipClasses, Instance.Orig->GetClass()->GetFName()))
			{
				continue;
			}

			if (DiffStructs(Instance.Orig, Instance.UPS, Instance.Id, /* in-out */ DiffCtx))
			{
				check(!DiffCtx.Diffs.IsEmpty());

				FDiffSet DiffSet = Flatten(DiffCtx.Diffs);
				check((GatherMode == EDiffGather::All) || (DiffSet.Num() == 1));

				GUpsDiff.Instances.Add(&Instance);
				PrintDiffs(/* out */ UpsDiffs, GUE.Names, DiffSet.Array());
				
				DiffCtx.Reset();
				UpsDiffs.Append(" in ");
				UpsDiffs.Append(Instance.Orig->GetFullName());
				UpsDiffs.Append("\n");	
				UpsDiffIdxs.Add(&Instance - &Instances[0]);
			}
		}
		GUpsDiff.Str = UpsDiffs.GetData();
		UE_LOGFMT(LogPlainPropsUObject, Display, "Detected {Diffs} diffs in {Objs} UObjects saved in a {KB}KB value stream using UPS", UpsDiffIdxs.Num(), Instances.Num(), Ups.Data.NumBytes() / 1024);
	}

	return NumPpDiffs;
}

////////////////////////////////////////////////////////////////////////////////

#if PP_WITH_ENGINE_HOOKS
struct FScopeSaveTestPackage
{
	FORCENOINLINE FScopeSaveTestPackage()
	{
		FSavePackageSettings& SavePackageSettings = FSavePackageSettings::GetDefaultSettings();
		ExternalWriteExportsFunc = SavePackageSettings.ExternalWriteExportsFunc;
		ExternalWriteFileFunc = SavePackageSettings.ExternalWriteFileFunc;

		SavePackageSettings.ExternalWriteExportsFunc = &FPlainPropsLinkerSandbox::SavePlainProps;
		SavePackageSettings.ExternalWriteFileFunc = &FPlainPropsLinkerSandbox::SavePackageToMemory;
	}

	~FScopeSaveTestPackage()
	{
		FSavePackageSettings& SavePackageSettings = FSavePackageSettings::GetDefaultSettings();
		SavePackageSettings.ExternalWriteExportsFunc = ExternalWriteExportsFunc;
		SavePackageSettings.ExternalWriteFileFunc = ExternalWriteFileFunc;
	}

	void(*ExternalWriteExportsFunc)(const FLinkerSave& LinkerSave) = nullptr;
	void(*ExternalWriteFileFunc)(const FLinkerSave& LinkerSave, uint8* Data, int64 Size) = nullptr;
};

struct FScopeLoadTestPackage
{
	FScopeLoadTestPackage(bool bInPlainProps)
		: SerializePropertiesFunc(GSerializePropertiesOverride)
		, SerializeObjectFunc(GSerializeObjectOverride)
		, LoadFromPackageFileFunc(GLoadFromPackageFileOverride)
		, bPlainProps(bInPlainProps)
	{
		if (bPlainProps)
		{
#if PP_OVERRIDE_OBJECT_LOADING
			GSerializeObjectOverride = &FPlainPropsLinkerSandbox::LoadObject;
#else
			GSerializePropertiesOverride = &FPlainPropsLinkerSandbox::LoadProperties;
#endif
		}
		GLoadFromPackageFileOverride = &FPlainPropsLinkerSandbox::OpenPackageReader;
	}

	~FScopeLoadTestPackage()
	{
		if (bPlainProps)
		{
			GSerializePropertiesOverride = SerializePropertiesFunc;
			GSerializeObjectOverride = SerializeObjectFunc;
		}
		GLoadFromPackageFileOverride = LoadFromPackageFileFunc;
	}

	bool (*SerializePropertiesFunc)(FStructuredArchiveSlot&, uint8*) = nullptr;
	bool (*SerializeObjectFunc)(UObject*, FArchive&) = nullptr;
	bool (*LoadFromPackageFileFunc)(const FPackagePath&, FArchive*&) = nullptr;
	bool bPlainProps;
};
#else
struct FScopeEmptyBase 
{
	FScopeEmptyBase() {}
	~FScopeEmptyBase() {}
};
struct FScopeSaveTestPackage : public FScopeEmptyBase {};
struct FScopeLoadTestPackage : public FScopeEmptyBase
{
	FScopeLoadTestPackage(bool) {};
};
#endif // PP_WITH_ENGINE_HOOKS

// Transforms /Game/Package_01 which is represented as "/Game/Package" and number 1 into being
// represented as "/Game/Package_01" and number 0, so that number 0 can be replace by a new number.
// This works for packages named: /Game/Package_01, /Game/Package_02, ..., /Game/Package_10
static FName GenerateDroppedPackageName(FName Name, int32 NewNumber = 0)
{
	if (NewNumber == 0)
	{
		return MakeUniqueObjectName(nullptr, UPackage::StaticClass(), NAME_TrashedPackage);
	}
	if (Name.GetNumber())
	{
		Name = FName(*Name.ToString(), 0);
	}
	return FName(Name, NewNumber);
}

static void DropPackages(TArray<UPackage*>&& Packages, FDropPackagesFunc DropPackagesFunc, int32 NewNumber = INDEX_NONE)
{
	for (UPackage* Package : Packages)
	{
		ForEachObjectWithPackage(Package, [](UObject* Object)
		{
			if (const UClass* Class = Cast<UClass>(Object))
			{
				FBindId Id = GUE.Names.IndexBindId(PlainProps::UE::IndexStruct(Class));
				if (GUE.Schemas.FindStruct(Id))
				{
					UE_LOGFMT(LogPlainPropsUObject, Log, " Dropping loaded dynamic class {BindId}", GUE.Debug.Print(Id));
					GUE.Schemas.DropStruct(Id);
					GUE.Metadatas.DropMetadata(Id);
				}
			}
			return true;
		});
	}

	DropPackagesFunc(MoveTemp(Packages), NewNumber);
}

int32 RoundtripViaLinkerBatch(
	TConstArrayView<UObject*> Objects,
	ERoundtrip Options,
	const FLinkerDiffFilter& DiffFilter,
	FDropPackagesFunc DropPackagesFunc)
{
#if PP_WITH_ENGINE_HOOKS
	TRACE_CPUPROFILER_EVENT_SCOPE(RoundtripViaLinkerBatch);

	TArray<FName> PackageNames;
	TArray<UPackage*> SavedPackages;
	TArray<UPackage*> BaselinePackages;
	TArray<UPackage*> PlainPropsPackages;

	{
		TArray<UObject*> AllPackages;
		// Collect all packages
		if (!Objects.Num())
		{
			GetObjectsOfClass(UPackage::StaticClass(), AllPackages);
		}
		// Collect specified packages
		if (Objects.Num())
		{
			for (UObject* Object : Objects)
			{
				AllPackages.AddUnique(Object->GetPackage());
			}
		}

		// Filter out in-memory packages
		// and pin the remaining packages and their objects because UPackage::Save may trigger GC after bp compilation
		for (UObject* PackageObj : AllPackages)
		{
			if (!PackageObj->HasAnyFlags(RF_WasLoaded))
			{
				continue;
			}
			UPackage* Package = Cast<UPackage>(PackageObj);
			if (Package->HasAnyPackageFlags(PKG_CompiledIn))
			{
				continue;
			}
			SavedPackages.Add(Package);
			PinPackage(Package);
		}
	}
	int32 NumPackages = SavedPackages.Num();

	FPlainPropsLinkerSandbox::Initialize(NumPackages);

	UE_LOGFMT(LogPlainPropsUObject, Display, "Saving {SaveNum} packages...", NumPackages);

	{
		FScopeSaveTestPackage SaveScope;

		TRACE_CPUPROFILER_EVENT_SCOPE(SavePackages);
		for (TArray<UPackage*>::TIterator It(SavedPackages); It; ++It)
		{
			UPackage* Package = *It;
			UObject* Asset = Package->FindAssetInPackage();
			FSavePackageArgs SaveArgs;
			SaveArgs.TopLevelFlags = RF_Standalone;
			SaveArgs.SaveFlags = SAVE_Async; // Write progress to temp memory instead of a temp file
											 // Always use either one of TPS (default) or UPS for saving the baseline package
			if (EnumHasAnyFlags(Options, ERoundtrip::UPS))
			{
				SaveArgs.SaveFlags |= SAVE_Unversioned_Properties;
			}
			FString SaveFilename = GetSavePackageFilename(Package);
			FSavePackageResultStruct Result = UPackage::Save(Package, Asset, *SaveFilename, SaveArgs);
			if (Result.Result == ESavePackageResult::Success)
			{
				PackageNames.Add(Package->GetFName());
			}
			else
			{
				UnpinPackage(Package);
				It.RemoveCurrent();
			}
		}
		NumPackages = SavedPackages.Num();

		// Unpin remaining packages, no more GC is expected, and some DropPackage callbacks tend to recreate uobjects.
		for (UPackage* Package : SavedPackages)
		{
			UnpinPackage(Package);
		}
	}

	// Roundtrip to text, also dumping all packages as yaml to disk
	if (EnumHasAnyFlags(Options, ERoundtrip::TextMemory | ERoundtrip::TextStable))
	{
		EMarkupFormat MarkupFormat = EnumHasAnyFlags(Options, ERoundtrip::JSON) ? EMarkupFormat::Json : EMarkupFormat::Yaml;
		TRACE_CPUPROFILER_EVENT_SCOPE(RoundtripToText);
		UE_LOGFMT(LogPlainPropsUObject, Display, "Roundtripping {SaveNum} packages to text...", NumPackages);
		for (FName PackageName : PackageNames)
		{
			FPlainPropsLinkerSandbox::RoundtripToTextFromMemory(PackageName, MarkupFormat);
		}
	}

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectPostCDOCompiled.AddStatic(FPlainPropsLinkerSandbox::OnBlueprintCDOCompiled);
#endif

	// Roundtrip plain props and diff against baseline (TPS or UPS)
	if (EnumHasAnyFlags(Options, ERoundtrip::PP))
	{
		TBitArray<> InstancedSet;
		InstancedSet.Reserve(NumPackages);
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(DropSavedPackages);
			UE_LOGFMT(LogPlainPropsUObject, Display, "Dropping {Num} saved packages...", NumPackages);
			DropPackages(CopyTemp(SavedPackages), DropPackagesFunc);
			// Some packages may have been reloaded as a result of the drop package callbacks,
			// mark that these should be loaded as instanced packages, i.e. with new unique in-memory package names
			for (FName PackageName : PackageNames)
			{
				InstancedSet.Add(FindObjectFast<UPackage>(/*Outer*/nullptr, PackageName) != nullptr);
			}
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MountPlainPropsPackages);
			UE_LOGFMT(LogPlainPropsUObject, Display, "Mounting {Num} plain props packages...", NumPackages);
			for (int32 I = 0; I < NumPackages; ++I)
			{
				PlainPropsPackages.Add(FPlainPropsLinkerSandbox::MountPlainPropsPackage(PackageNames[I], InstancedSet[I]));
			}
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(LoadPlainPropsPackages);
			UE_LOGFMT(LogPlainPropsUObject, Display, "Loading {Num} plain props packages...", NumPackages);
			FScopeLoadTestPackage LoadScope(/*bPlainProps*/true);
			double Start = FPlatformTime::Seconds();
			for (UPackage* Package : PlainPropsPackages)
			{
				FPlainPropsLinkerSandbox::LoadPlainPropsPackage(Package);
			}
			UE_LOGF(LogPlainPropsUObject, Display, "Loaded %d plainprops packages in %.2f s",
				NumPackages, FPlatformTime::Seconds() - Start);
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(DropPlainPropsPackages);
			UE_LOGFMT(LogPlainPropsUObject, Display, "Dropping {Num} plain props packages...", NumPackages);
			DropPackages(CopyTemp(PlainPropsPackages), DropPackagesFunc, 8849);
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MountPlainPropsPackages);
			UE_LOGFMT(LogPlainPropsUObject, Display, "Mounting {Num} baseline packages...", NumPackages);
			for (int32 I = 0; I < NumPackages; ++I)
			{
				BaselinePackages.Add(FPlainPropsLinkerSandbox::MountBaselinePackage(PackageNames[I], InstancedSet[I]));
			}
		}
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(LoadBaselinePackages);
			UE_LOGFMT(LogPlainPropsUObject, Display, "Loading {Num} baseline packages...", NumPackages);
			FScopeLoadTestPackage LoadScope(/*bPlainProps*/false);
			double Start = FPlatformTime::Seconds();
			for (UPackage* Package : BaselinePackages)
			{
				FPlainPropsLinkerSandbox::LoadBaselinePackage(Package);
			}
			UE_LOGF(LogPlainPropsUObject, Display, "Loaded %d baseline packages in %.2f s",
				NumPackages, FPlatformTime::Seconds() - Start);
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(DropBaselinePackages);
			UE_LOGFMT(LogPlainPropsUObject, Display, "Dropping {Num} baseline packages...", NumPackages);
			DropPackages(CopyTemp(BaselinePackages), DropPackagesFunc, 8848);
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(DiffPackages);
			UE_LOGFMT(LogPlainPropsUObject, Display, "Diffing {Num} packages...\n", NumPackages);

			EParallelForFlags Flags = EParallelForFlags::Unbalanced;
			Flags |= FPlatformMisc::IsDebuggerPresent() ? EParallelForFlags::ForceSingleThread : EParallelForFlags::None;

			FDiffObjectFilter Filter = 
			{
				DiffFilter.BypassNativeIdenticalStructs,
				DiffFilter.IgnoreStructs,
				DiffFilter.IgnorePropertiesForStructs,
				DiffFilter.IgnorePropertiesForBases,
				DiffFilter.IgnoreCastFlags,
			};

			std::atomic<uint32> NumDiffs = 0;
			std::atomic<uint32> NumIgnored = 0;
			ParallelFor(TEXT("DiffPackage"), NumPackages, /*MinBatchSize*/1,
				[&NumDiffs, &NumIgnored, &PackageNames, &BaselinePackages, &PlainPropsPackages, &Filter, Options](int32 I)
			{
				FDiffObjectContext Ctx = { BaselinePackages[I], PlainPropsPackages[I], Filter };
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(DiffObjects);
					DiffObjects(Ctx);
				}
				if (Ctx.Diffs.Num())
				{
					TUtf8StringBuilder<2048> Diff;
					PrintDiff(Diff, Ctx);
					UE_LOGFMT(LogPlainPropsUObject, Display, "");
					UE_LOGFMT(LogPlainPropsUObject, Display, "Detected diff in package {PackageName}:\n{Diff}",
						PackageNames[I], Diff.ToString());
					++NumDiffs;

					{
						TRACE_CPUPROFILER_EVENT_SCOPE(PrintToMarkup);
						EMarkupFormat MarkupFormat = EnumHasAnyFlags(Options, ERoundtrip::JSON) ? EMarkupFormat::Json : EMarkupFormat::Yaml;
						FPlainPropsLinkerSandbox::PrintPackageToFile(PackageNames[I], MarkupFormat);
					}

					{
						TRACE_CPUPROFILER_EVENT_SCOPE(DumpToJson);
						const FDiffObjectNode& RootNode = Ctx.Diffs.Last();
						const UObject* A = static_cast<const UObject*>(RootNode.A);
						const UObject* B = static_cast<const UObject*>(RootNode.B);
						if (A)
						{
							DumpUObjectToJsonFile(A);
						}
						if (B)
						{
							DumpUObjectToJsonFile(B);
						}
					}
				}
				else if (Ctx.NumIgnoredDiffs)
				{
					++NumIgnored;
				}

				if (Ctx.OverrideDiffs.Num())
				{
					UE_LOGFMT(LogPlainPropsUObject, Display, "");
					UE_LOGFMT(LogPlainPropsUObject, Display, "Detected override diff in package {PackageName}",
						PackageNames[I]);

					FOverridableManager& OverrideManager = FOverridableManager::Get();

					for (FDiffOverride& Diff : Ctx.OverrideDiffs)
					{
						const FOverriddenPropertySet* OverridesA = OverrideManager.GetOverriddenProperties(Diff.A);
						const FOverriddenPropertySet* OverridesB = OverrideManager.GetOverriddenProperties(Diff.B);

						UE_LOGF(LogPlainPropsUObject, Display, "A: %ls", *Diff.A->GetPathName());
						UE_LOGF(LogPlainPropsUObject, Display, "A overrides:");
						UE_LOGF(LogPlainPropsUObject, Display, "\n%ls", OverridesA ? *OverridesA->ToDebugString() : TEXT("NULL"));
						UE_LOGF(LogPlainPropsUObject, Display, "B overrides");
						UE_LOGF(LogPlainPropsUObject, Display, "\n%ls", OverridesB ? *OverridesB->ToDebugString() : TEXT("NULL"));
					}
				}
			}, Flags);
			UE_LOGFMT(LogPlainPropsUObject, Display, "");
			UE_LOGFMT(LogPlainPropsUObject, Display, "Ignored {NumIgnored} package diff(s) in {Num} package(s)", NumIgnored, NumPackages);
			UE_LOGFMT(LogPlainPropsUObject, Display, "Detected {NumDiffs} package diff(s) in {Num} package(s)", NumDiffs, NumPackages);
		}
	}

	FPersistentBatchLoader::PrintDebugInfo();
	TRACE_CPUPROFILER_EVENT_FLUSH();
	
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CollectGarbage);
		UE_LOGFMT(LogPlainPropsUObject, Log, "Collecting garbage...");
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}

#if UE_BUILD_DEBUG
	// Note: Shutdown is too slow to be enabled by default.
	// It will reset and flush all the linkers which loads and detaches all bulk data from the saved memory archive.
	// This takes ~15s for ~3000 packages in Release config.
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Shutdown);
		FScopeLoadTestPackage LoadScope(/*bPlainProps*/false);
		FPlainPropsLinkerSandbox::Shutdown();
	}
#endif

#endif // PP_WITH_ENGINE_HOOKS
	return 0;
}

bool SaveTestPackage(UPackage* Package, const FString& Filename, const FSavePackageArgs& SaveArgs)
{
	bool bDidSave = false;

#if PP_WITH_ENGINE_HOOKS
	check(Package);
	checkf(SaveArgs.SaveFlags & SAVE_Async, TEXT("Only packages saved to a temporary memory writer are currently supported"));

	FScopeSaveTestPackage SaveScope;

	// We need a valid path to be set on the package for when we load it back.
	FPackagePath PackagePath;
	PackagePath = FPackagePath::FromLocalPath(Filename);
	Package->SetLoadedPath(PackagePath);

	const FSavePackageResultStruct SavePackageResult = UPackage::Save(Package, nullptr, *Filename, SaveArgs);

	IFileManager& FileManager = IFileManager::Get();

	if (SavePackageResult.IsSuccessful())
	{
		// Because we have overridden ExternalWriteFileFunc to only write to memory
		// (see FScopeSaveTestPackage) the previous save did not generate an actual file.
		// This will cause our package to be considered missing during load (see
		// TryGetPackageInfoFromPackageStore). As a workaround we create a temporary
		// empty package file.
		if (TUniquePtr<FArchive> Ar(FileManager.CreateFileWriter(*Filename)); Ar)
		{
			uint32 TempData = 0;
			Ar->Serialize(&TempData, sizeof(TempData));
		}
	}

	bDidSave = SavePackageResult.IsSuccessful() && FileManager.FileExists(*Filename);
#endif

	return bDidSave;
}

UPackage* LoadTestPackage(const FString& Filename)
{
	UPackage* Package = nullptr;

#if PP_WITH_ENGINE_HOOKS
	FScopeLoadTestPackage LoadScope(/*bPlainProps*/true);

	FName PackageName = FName(*Filename);

	Package = FPlainPropsLinkerSandbox::MountPlainPropsPackage(PackageName, /*bClone*/false);
	FPlainPropsLinkerSandbox::LoadPlainPropsPackage(Package);
#endif

	return Package;
}

} // namespace PlainProps::UE
