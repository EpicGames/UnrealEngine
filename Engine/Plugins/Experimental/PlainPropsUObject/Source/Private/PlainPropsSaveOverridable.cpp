// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsSaveOverridableInternal.h"
#include "PlainPropsRestoreOverridableInternal.h"
#include "PlainPropsInternalPrivateMemberPtr.h"
#include "PlainPropsSave.h"
#include "PlainPropsSaveMember.h"
#include "PlainPropsVisitMember.h"
#include "PlainPropsUObjectRuntime.h"
#include "UObject/OverriddenPropertySet.h"

PP_DEFINE_PRIVATE_MEMBER_NAMESPACE_BEGIN(FOverriddenPropertyNodeID_Private)
PP_DEFINE_PRIVATE_MEMBER_PTR(FOverriddenPropertyNodeID, Path, FOverriddenPropertyPath);
PP_DEFINE_PRIVATE_MEMBER_NAMESPACE_END()

PP_DEFINE_PRIVATE_MEMBER_NAMESPACE_BEGIN(FOverriddenPropertyPath_Private)
PP_DEFINE_PRIVATE_MEMBER_PTR(FOverriddenPropertyPath, Path, TArray<FName>);
PP_DEFINE_PRIVATE_MEMBER_NAMESPACE_END()

namespace PlainProps::UE
{

// Todo: This should be moved to an appropriate context that we can pass around.
thread_local const FOverriddenPropertySet* TlsSaveOverrides = nullptr;

FName GetOverridePropertyName(const FOverriddenPropertyNode& Override)
{
	FName Name;
	const FOverriddenPropertyPath* Path = nullptr;
	{
		using namespace FOverriddenPropertyNodeID_Private;
		Path = &(Override.GetNodeID().*_Path);
	}
	{
		using namespace FOverriddenPropertyPath_Private;
		Name = Path->IsEmpty() ? FName() : ((*Path).*_Path)[0];
	}
	return Name;
}

inline const FOverriddenPropertyNode* FindSubOverride(const FOverriddenPropertyNode& Override, FName MemberName)
{
	for (const FOverriddenPropertyNode& SubNode : Override.GetSubPropertyNodes())
	{
		FName IdName = GetOverridePropertyName(SubNode);
		if (IdName == MemberName)
		{
			return &SubNode;
		}
	}
	return nullptr;
}

inline const uint8* At(const void* Ptr, SIZE_T Offset)
{
	return static_cast<const uint8*>(Ptr) + Offset;
}

inline bool IsOverridden(const FOverriddenPropertyNode& Override)
{
	EOverriddenPropertyOperation Op = Override.GetOperation();
	return (Op != EOverriddenPropertyOperation::None) && (Op != EOverriddenPropertyOperation::SubObjectsShadowing);
}

inline void SaveNamedMembers(FMemberBuilder& Out, const void* Struct, FMemberVisitor It, TConstArrayView<FMemberId> Names, const FSaveContext& Ctx)
{
	for (FMemberId Name : Names)
	{
		switch (It.PeekKind())
		{
		case EMemberKind::Leaf:		SaveMember(Out, Struct, Name, Ctx, It.GrabLeaf());		break;
		case EMemberKind::Range:	SaveMember(Out, Struct, Name, Ctx, It.GrabRange());		break;
		case EMemberKind::Struct:	SaveMember(Out, Struct, Name, Ctx, It.GrabStruct());	break;
		}
	}
	checkSlow(!It.HasMore());
}

static void SaveRangeMember(FMemberBuilder& Out, const void* Struct, FMemberId Name, const FSaveContext& Ctx, FRangeMemberBinding Member)
{
	Out.AddRange(Name, { CreateRangeSchema(Ctx.Scratch, Member), SaveRange(At(Struct, Member.Offset), Member, Ctx) });
}

static const FStructDeclaration& SaveStructOverrides(FMemberBuilder& Out, const void* Struct, const FOverriddenPropertyNode& Override, FBaseline Base, FBindId Id, const FSaveContext& Ctx, bool bIsRoot);

static void SaveStructMember(FMemberBuilder& Out, const void* Struct, const FOverriddenPropertyNode& Override, 
							 const void* Default, FMemberId Name, const FSaveContext& Ctx, FStructMemberBinding Member)
{
	FMemberBuilder TempBuilder;
	const void* MemberDefault = Default ? At(Default, Member.Offset) : nullptr;
	const FStructDeclaration& Declaration = SaveStructOverrides(TempBuilder, At(Struct, Member.Offset), Override, MemberDefault, Member.Id, Ctx, false);

	FBuiltStruct* Result = TempBuilder.IsEmpty() ? nullptr : TempBuilder.BuildAndReset(Ctx.Scratch, Declaration, Ctx.Schemas.GetDebug());

	if (Result)
	{
		Out.AddStruct(Name, Member.Id, Result);
	}
}

static void SaveSchemaBoundStructOverrides(FMemberBuilder& Out, const void* Struct, const FOverriddenPropertyNode& Override,
											FBaseline Base, const FSchemaBinding& Schema, FBindId StructId, const FStructDeclaration& Declaration, 
											const FSaveContext& Ctx, bool bIsRoot)
{
	static FOverriddenPropertyNode DummyReplaceOverride = []() 
	{ 
		FOverriddenPropertyNode Node; 
		ResetOverriddenPropertyNode(Node, EOverriddenPropertyOperation::Replace, {});
		return Node;
	}();

	FMemberVisitor It(Schema);
	if (Declaration.Super)
	{
		FBindId SuperId = It.GrabSuper();
		checkSlow(SuperId == ToOptionalStruct(Declaration.Super));
		const FStructDeclaration* SuperDecl;
		const FSchemaBinding& SuperSchema = Ctx.Schemas.GetStruct(SuperId, /* out */ SuperDecl);
		SaveSchemaBoundStructOverrides(Out, Struct, Override, Base.Super(), SuperSchema, SuperId, *SuperDecl, Ctx, bIsRoot);
		Out.BuildSuperStruct(Ctx.Scratch, *SuperDecl, Ctx.Schemas.GetDebug());
	}

	// The override operation for root object doesn't cover always overridden properties, hence the bIsRoot check
	if (IsOverridden(Override) || bIsRoot)
	{
		if (Declaration.Occupancy == EMemberPresence::AllowSparse)
		{
			TConstArrayView<EPropertyFlags> MemberFlags = GUE.Metadatas.GetMemberFlags(StructId);
			TConstArrayView<FMemberId> MemberNames = Declaration.GetMemberOrder();
			for (int32 Index = 0; Index < MemberNames.Num(); ++Index)
			{
				EPropertyFlags Flags = MemberFlags[Index];
				
				if ((Flags & CPF_ExperimentalNeverOverriden) != 0)
				{
					It.SkipMember();
					continue;
				}

				FMemberId Name = MemberNames[Index];
				const FOverriddenPropertyNode* SubOverride = nullptr;
				if ((Flags & CPF_ExperimentalAlwaysOverriden) != 0)
				{
					SubOverride = &DummyReplaceOverride;
				}
				else if (Override.GetOperation() == EOverriddenPropertyOperation::Replace)
				{
					// Keep using the same override since a replace operation implies no sub-nodes.
					SubOverride = &Override;
				}
				else
				{
					FName MemberFName = GUE.Names.ResolveName(Name.Id).ToName();
					SubOverride = FindSubOverride(Override, MemberFName);
				}

				if (SubOverride)
				{
					// The default is only used for custom bindings and specifically overridable container bindings.
					// When the property operation is 'Replace' we pass no default so that the bindings for overridable
					// containers do not consider individual items but instead a single 'Assign' operation.
					const void* Default = (SubOverride->GetOperation() == EOverriddenPropertyOperation::Replace)
											? nullptr : Base.Get();

					switch (It.PeekKind())
					{
					case EMemberKind::Leaf:		SaveMember(Out, Struct, Name, Ctx, It.GrabLeaf());	break;
					case EMemberKind::Range:	SaveRangeMember(Out, Struct, Name, Ctx, It.GrabRange());	break;
					case EMemberKind::Struct:	SaveStructMember(Out, Struct, *SubOverride, Default, Name, Ctx, It.GrabStruct()); break;
					}
				}
				else
				{
					It.SkipMember();
				}
			}
			checkSlow(!It.HasMore());
		}
		else
		{
			SaveNamedMembers(Out, Struct, It, Declaration.GetMemberOrder(), Ctx);
		}
	}
}

static const FStructDeclaration& SaveStructOverrides(FMemberBuilder& Out, const void* Struct, const FOverriddenPropertyNode& Override,
													 FBaseline Base, FBindId Id, const FSaveContext& Ctx, bool bIsRoot)
{
	const FStructDeclaration* Declaration = nullptr;
	if (ICustomBinding* Custom = Ctx.Customs.FindStructToSave(Id, /* out */ Declaration))
	{
		if (IsOverridden(Override))
		{
			Custom->SaveCustom(Out, Struct, Base, Ctx);
		}
	}
	else
	{
		const FSchemaBinding& Schema = Ctx.Schemas.GetStruct(Id, /* out */ Declaration);
		SaveSchemaBoundStructOverrides(Out, Struct, Override, Base, Schema, Id, *Declaration, Ctx, bIsRoot);
	}

	return *Declaration;
}

static FBuiltStruct* SaveStructOverrides(const void* Struct, const FOverriddenPropertyNode& Override, FBaseline Base, FBindId Id, const FSaveContext& Ctx, bool bIsRoot)
{
	FMemberBuilder Out;
	const FStructDeclaration& Declaration = SaveStructOverrides(Out, Struct, Override, Base, Id, Ctx, bIsRoot);
	return Out.BuildAndReset(Ctx.Scratch, Declaration, Ctx.Schemas.GetDebug());
}

FBuiltStruct* SaveStructOverrides(const FOverriddenPropertySet& Overrides, 
								  FBaseline Base, FBindId BindId, const FSaveContext& Ctx)
{
	const FOverriddenPropertySet* PrevOverrides = TlsSaveOverrides;
	TlsSaveOverrides = &Overrides;
	
	const FOverriddenPropertyNode& RootOverride = Overrides.GetRootOverriddenPropertyNode();

	FBuiltStruct* Result = SaveStructOverrides(Overrides.GetOwner(), RootOverride, Base, BindId, Ctx, true);

	TlsSaveOverrides = PrevOverrides;

	return Result;
}

const FOverriddenPropertySet* GetSaveOverrides()
{
	return TlsSaveOverrides;
}

} // namespace PlainProps::UE
