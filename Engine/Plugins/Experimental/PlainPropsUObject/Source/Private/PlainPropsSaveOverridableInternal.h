// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsTypes.h"
#include "PlainPropsSave.h"

struct FOverriddenPropertySet;
struct FOverriddenPropertyNode;

namespace PlainProps
{

namespace UE
{

FName GetOverridePropertyName(const FOverriddenPropertyNode& Override);

FBuiltStruct* SaveStructOverrides(const FOverriddenPropertySet& Overrides, FBaseline Base, FBindId BindId, const FSaveContext& Context);

const FOverriddenPropertySet* GetSaveOverrides();

} // namespace UE
} // namespace PlainProps
