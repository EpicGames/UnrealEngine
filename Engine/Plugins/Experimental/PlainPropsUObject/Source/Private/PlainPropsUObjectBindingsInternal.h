// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsTypes.h"
#include "PlainPropsBind.h"
#include "Logging/LogMacros.h"

DECLARE_LOG_CATEGORY_EXTERN(LogPlainPropsUObject, Log, All);

class FField;
class FProperty;
class UStruct;
namespace PlainProps { struct FSaveContext; };

namespace PlainProps::UE
{

FType			IndexStruct(const UStruct* Struct);
bool			ShouldBind(const UStruct* Struct);
const UStruct*	SkipEmptyBases(const UStruct* Struct);

// Helpers for native UField and FProperty meta bindings
FType			IndexStructMeta(const UStruct* Struct);
void 			BindStructMeta(FBindId Id, const UStruct* Struct);
FDualStructId 	IndexPropertyMeta(const FField* Property);

} // namespace PlainProps::UE

