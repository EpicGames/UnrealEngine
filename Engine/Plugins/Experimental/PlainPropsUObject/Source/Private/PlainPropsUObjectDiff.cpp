// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsUObjectDiffInternal.h"
#include "PlainPropsInternalPrivateMemberPtr.h"
#include "PlainPropsSaveOverridableInternal.h"
#include "UObject/Class.h"
#include "UObject/FieldPathProperty.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"
#include "UObject/OverridableManager.h"
#include "Containers/StringFwd.h"
#include "StructUtils/InstancedStruct.h"

// Temp hacks for non-intrusive prototype
PP_DEFINE_PRIVATE_MEMBER_NAMESPACE_BEGIN(FFieldPath_Private)
PP_DEFINE_PRIVATE_MEMBER_PTR(FFieldPath, ResolvedOwner,	TWeakObjectPtr<UStruct>);
PP_DEFINE_PRIVATE_MEMBER_PTR(FFieldPath, Path,			TArray<FName>);
PP_DEFINE_PRIVATE_MEMBER_NAMESPACE_END()

PP_DEFINE_PRIVATE_MEMBER_NAMESPACE_BEGIN(FOverriddenPropertyNodeIdDiff_Private)
PP_DEFINE_PRIVATE_MEMBER_PTR(FOverriddenPropertyNodeID, Object, TWeakObjectPtr<const UObject>);
PP_DEFINE_PRIVATE_MEMBER_NAMESPACE_END()

namespace PlainProps::UE
{

template<uint64 Mask>
static bool HasAnyCastClassFlags(uint64 Flags)
{
	return (Mask & Flags) != 0;
}

// Inspired by UObjectBaseUtility::GetPathName and UObjectBaseUtility::GetFullGroupName.
// Adjusted to handle the Object->HasAnyFlags(RF_HasExternalPackage) scenario.
void GetRelativePathName(const UObject* Object, FStringBuilderBase& Out)
{
	const UObject* Outer = Object->GetOuter();
	if (Outer)
	{
		if (const UObject* NextOuter = Outer->GetOuter())
		{
			GetRelativePathName(Outer, Out);

			// SUBOBJECT_DELIMITER_CHAR is used to indicate that this object's outer is not a UPackage
			if (Outer->GetClass() != UPackage::StaticClass() && NextOuter->GetClass() == UPackage::StaticClass())
			{
				Out << SUBOBJECT_DELIMITER_CHAR;
			}
			else
			{
				Out << TEXT('.');
			}
		}
		Object->GetFName().AppendString(Out);
	}
}

FString GetRelativePathName(const UObject* Object)
{
	TStringBuilder<256> Out;
	GetRelativePathName(Object, Out);
	return FString(FStringView(Out));
}

FAnsiStringView ToString(EDiffObjectNodeType Type)
{
	FAnsiStringView Types[5] = { ("NotInA"), ("NotInB"), ("Class"), ("Object"), ("Property") };
	return Types[(uint8)Type];
}

// Slow FName based IsChildOf implementation that does not require any dependencies
static bool IsChildOf(const UStruct* Struct, FName BaseName)
{
	for (const UStruct* It : Struct->GetSuperStructIterator())
	{
		if (It->GetFName() == BaseName)
		{
			return true;
		}
	}
	return false;
}

static bool IgnoreDiff(const void* A, const void* B, const FProperty* Property, FDiffObjectContext& Ctx)
{
	const FDiffObjectFilter& Flt = Ctx.Filter;
	const uint64 CastFlags = Property->GetCastFlags();
	const FName PropertyName = Property->GetFName();
	const UStruct* OwnerStruct = Property->GetOwnerStruct();

	auto ReturnDiff = [&Ctx](bool bDiff) FORCENOINLINE
	{
		// place breakpoint(s) here
		if (bDiff)
		{
			++Ctx.NumIgnoredDiffs;
			return true;
		}
		return false;
	};

	// 1. Silence diffs for FProperty types missing a custom PlainPropsUObjectDiff implementation

	if (CastFlags & Flt.IgnoreCastFlags)
	{
		return ReturnDiff(true);
	}

	// 2. Silence diffs for missing owner structs and unique runtime derived properties

	if (Flt.IgnoreStructs.Contains(OwnerStruct->GetFName()))
	{
		return ReturnDiff(true);
	}

	if (Flt.IgnorePropertiesForStructs.Contains(TPair<FName, FName>{ PropertyName, OwnerStruct->GetFName() }))
	{
		return ReturnDiff(true);
	}

	// 3. Silence diffs for unique runtime derived properties for some specific base classes

	if (const FName* BaseName = Flt.IgnorePropertiesForBases.Find(PropertyName))
	{
		return ReturnDiff(IsChildOf(OwnerStruct, *BaseName));
	}

	// 4. Silence diffs for missing structs in struct properties (inside containers)
	
	TArray<FName, TInlineAllocator<2>> StructNames;
	if (const FStructProperty* StructProp = CastField<FStructProperty>(Property))
	{
		StructNames.Add(StructProp->Struct->GetFName());
	}
	else if (const FArrayProperty* ArrayProp = CastField<FArrayProperty>(Property))
	{
		if (const FStructProperty* InnerProp = CastField<FStructProperty>(ArrayProp->Inner))
		{
			StructNames.Add(InnerProp->Struct->GetFName());
		}
	}
	else if (const FMapProperty* MapProp = CastField<FMapProperty>(Property))
	{
		if (const FStructProperty* KeyProp = CastField<FStructProperty>(MapProp->GetKeyProperty()))
		{
			StructNames.Add(KeyProp->Struct->GetFName());
		}
		if (const FStructProperty* ValueProp = CastField<FStructProperty>(MapProp->GetValueProperty()))
		{
			StructNames.Add(ValueProp->Struct->GetFName());
		}
	}
	for (FName StructName : StructNames)
	{
		if (Flt.IgnoreStructs.Contains(StructName))
		{
			return ReturnDiff(true);
		}
	}

	return ReturnDiff(false);
};

///////////////////////////////////////////////////////////////////////////////

bool DiffStructs(   const void* A, const void* B, const UStruct*   Struct,   FDiffObjectContext& Ctx);
bool DiffProperties(const void* A, const void* B, const FProperty* Property, FDiffObjectContext& Ctx);

bool DiffObjectReferences(const UObject* A, const UObject* B, FDiffObjectContext& Ctx)
{
	if (A == B)
	{
		return false;
	}

	const UClass* Class = A ? A->GetClass() : B->GetClass();
	if (!A || !B)
	{
		// place breakpoint here
		Ctx.Diffs.Emplace(!A ? EDiffObjectNodeType::NotInA : EDiffObjectNodeType::NotInB, A, B, Class);
		return true;
	}

	bool bAlreadyInSet;
	Ctx.Visited.Add({A, B}, &bAlreadyInSet);
	if (bAlreadyInSet)
	{
		return false;
	}

	if (Class != B->GetClass())
	{
		// Treat dynamic (non-native) classes as equal if their properties are equal.
		if (Class->HasAnyClassFlags(CLASS_Native) ||
			B->GetClass()->HasAnyClassFlags(CLASS_Native) ||
			DiffObjectReferences(Class, B->GetClass(), Ctx))
		{
			Ctx.Diffs.Emplace(EDiffObjectNodeType::Class, A, B, Class);
			return true;
		}
	}

	bool bIsInA = A->IsIn(Ctx.RootA);
	bool bIsInB = B->IsIn(Ctx.RootB);
	if (bIsInA & bIsInB)
	{
		TStringBuilder<256> PathA;
		TStringBuilder<256> PathB;
		GetRelativePathName(A, PathA);
		GetRelativePathName(B, PathB);
		bool bDiff = FStringView(PathA) != FStringView(PathB);
		if (bDiff)
		{
			// place breakpoint here
			Ctx.Diffs.Emplace(EDiffObjectNodeType::Object, A, B, Class);
		}
		return bDiff;
	}
	else if (bIsInA | bIsInB)
	{
		Ctx.Diffs.Emplace(EDiffObjectNodeType::Object, A, B, Class);
		return true;
	}
	return DiffStructs(A, B, Class, Ctx);
}

bool DiffInstancedStructs(const FInstancedStruct* A, const FInstancedStruct* B, const FStructProperty* Property, FDiffObjectContext& Ctx)
{
	if (A == B)
	{
		return false;
	}

	if (!A || !B || A->GetScriptStruct() != B->GetScriptStruct())
	{
		Ctx.Diffs.Emplace(EDiffObjectNodeType::Property, A, B, Property);
		return true;
	}
	else if (!A->GetScriptStruct())
	{
		return false;
	}

	return DiffStructs(A->GetMemory(), B->GetMemory(), A->GetScriptStruct(), Ctx);
}

bool DiffSoftObjectPaths(const FSoftObjectPath* A, const FSoftObjectPath* B, const FStructProperty* Property, FDiffObjectContext& Ctx)
{
	if (A == B)
	{
		return false;
	}

	if (!A || !B)
	{
		Ctx.Diffs.Emplace(EDiffObjectNodeType::Property, A, B, Property);
		return true;
	}

	bool bDiff;
	bool bIsInA = A->GetLongPackageFName() == Ctx.RootA->GetFName();
	bool bIsInB = B->GetLongPackageFName() == Ctx.RootB->GetFName();
	if (bIsInA & bIsInB)
	{
		bDiff = A->GetAssetFName() != B->GetAssetFName() || A->GetSubPathUtf8String() != B->GetSubPathUtf8String();
	}
	else if (bIsInA | bIsInB)
	{
		bDiff = true;
	}
	else
	{
		bDiff = *A != *B;
	}
	if (bDiff)
	{
		Ctx.Diffs.Emplace(EDiffObjectNodeType::Property, A, B, Property);
	}
	return bDiff;
}

bool DiffObjects(const UObject* A, const UObject* B, FDiffObjectContext& Ctx)
{
	if (A == B)
	{
		return false;
	}

	const UClass* Class = A ? A->GetClass() : B->GetClass();
	if (!A || !B)
	{
		Ctx.Diffs.Emplace(EDiffObjectNodeType::Object, A, B, Class);
		return true;
	}

	if (Class != B->GetClass())
	{
		// Treat dynamic (non-native) classes as equal if their properties are equal.
		if (Class->HasAnyClassFlags(CLASS_Native) ||
			B->GetClass()->HasAnyClassFlags(CLASS_Native) ||
			DiffObjectReferences(Class, B->GetClass(), Ctx))
		{
			Ctx.Diffs.Emplace(EDiffObjectNodeType::Class, A, B, Class);
			return true;
		}
	}

	if (DiffStructs(A, B, Class, Ctx))
	{
		Ctx.Diffs.Emplace(EDiffObjectNodeType::Object, A, B, Class);
		return true;
	}

	return false;
}

bool DiffStructProperties(const void* A, const void* B, const FStructProperty* Property, FDiffObjectContext& Ctx)
{
	static const FName SoftObjectPath{"SoftObjectPath"};

	const UScriptStruct* ScriptStruct = Property->Struct;
	if (ScriptStruct->GetFName() == SoftObjectPath)
	{
		return DiffSoftObjectPaths(static_cast<const FSoftObjectPath*>(A), static_cast<const FSoftObjectPath*>(B), Property, Ctx);
	}

	static const FName InstancedStruct{"InstancedStruct"};
	if (ScriptStruct->GetFName() == InstancedStruct)
	{
		return DiffInstancedStructs(static_cast<const FInstancedStruct*>(A), static_cast<const FInstancedStruct*>(B), Property, Ctx);
	}

	// see ScriptStruct->CompareScriptStruct(A, B, /*PortFlags*/ 0);
	if (ScriptStruct->StructFlags & STRUCT_IdenticalNative)
	{
		if (Ctx.Filter.BypassNativeIdenticalStructs.Contains(ScriptStruct->GetFName()))
		{
			; // fall through to default struct diffing doing one property at a time
		}
		else
		{
			bool bResult;
			if (ScriptStruct->GetCppStructOps()->Identical(A, B, /*PortFlags*/0, bResult))
			{
				if (!bResult)
				{
					if (IgnoreDiff(A, B, Property, Ctx))
					{
						return false;
					}
					Ctx.Diffs.Emplace(EDiffObjectNodeType::Property, A, B, Property);
				}
				return !bResult;
			}
		}
	}
	if (DiffStructs(A, B, ScriptStruct, Ctx))
	{
		if (IgnoreDiff(A, B, Property, Ctx))
		{
			Ctx.Diffs.Reset();
			return false;
		}

		Ctx.Diffs.Emplace(EDiffObjectNodeType::Property, A, B, Property);
		return true;
	}
	return false;
}

bool DiffObjectProperties(const void* A, const void* B, const FObjectPropertyBase* Property, FDiffObjectContext& Ctx)
{
	const UObject* ValueA = Property->GetObjectPropertyValue(A);
	const UObject* ValueB = Property->GetObjectPropertyValue(B);
	if (DiffObjectReferences(ValueA, ValueB, Ctx))
	{
		Ctx.Diffs.Emplace(EDiffObjectNodeType::Property, A, B, Property);
		return true;
	}
	return false;
}

bool DiffArrayProperties(const void* A, const void* B, const FArrayProperty* Property, FDiffObjectContext& Ctx)
{
	FScriptArrayHelper ArrayA(Property, A);
	FScriptArrayHelper ArrayB(Property, B);

	if (ArrayA.Num() != ArrayB.Num())
	{
		if (IgnoreDiff(A, B, Property, Ctx))
		{
			return false;
		}
		Ctx.Diffs.Emplace(EDiffObjectNodeType::Property, A, B, Property);
		return true;
	}

	for (int32 I = 0; I < ArrayA.Num(); I++)
	{
		const void* ValueA = ArrayA.GetRawPtr(I);
		const void* ValueB = ArrayB.GetRawPtr(I);
		if (DiffProperties(ValueA, ValueB, Property->Inner, Ctx))
		{
			Ctx.Diffs.Emplace(EDiffObjectNodeType::Property, A, B, Property, I);
			return true;
		}
	}

	return false;
}

bool DiffSetProperties(const void* A, const void* B, const FSetProperty* Property, FDiffObjectContext& Ctx)
{
	FScriptSetHelper SetA(Property, A);
	FScriptSetHelper SetB(Property, B);
	if (SetA.Num() != SetB.Num())
	{
		if (IgnoreDiff(A, B, Property, Ctx))
		{
			return false;
		}
		Ctx.Diffs.Emplace(EDiffObjectNodeType::Property, A, B, Property);
		return true;
	}

	FScriptSetHelper::FIterator ItA(SetA);
	FScriptSetHelper::FIterator ItB(SetB);
	uint64 Idx = 0;
	for (; ItA && ItB; ++ItA, ++ItB, ++Idx)
	{
		const void* ValueA = SetA.GetElementPtr(ItA);
		const void* ValueB = SetB.GetElementPtr(ItB);
		if (DiffProperties(ValueA, ValueB, Property->ElementProp, Ctx))
		{
			Ctx.Diffs.Emplace(EDiffObjectNodeType::Property, A, B, Property, Idx);
			return true;
		}
	}
	return false;
}

bool DiffMapProperties(const void* A, const void* B, const FMapProperty* Property, FDiffObjectContext& Ctx)
{
	FScriptMapHelper MapA(Property, A);
	FScriptMapHelper MapB(Property, B);
	if (MapA.Num() != MapB.Num())
	{
		if (IgnoreDiff(A, B, Property, Ctx))
		{
			return false;
		}
		Ctx.Diffs.Emplace(EDiffObjectNodeType::Property, A, B, Property);
		return true;
	}
	FScriptMapHelper::FIterator ItA(MapA);
	FScriptMapHelper::FIterator ItB(MapB);
	uint64 Idx = 0;
	for (; ItA && ItB; ++ItA, ++ItB, ++Idx)
	{
		const void* KeyA = MapA.GetKeyPtr(ItA);
		const void* KeyB = MapB.GetKeyPtr(ItB);
		if (DiffProperties(KeyA, KeyB, Property->KeyProp, Ctx))
		{
			Ctx.Diffs.Emplace(EDiffObjectNodeType::Property, A, B, Property, Idx);
			return true;
		}
		const void* ValA = MapA.GetValuePtr(ItA);
		const void* ValB = MapB.GetValuePtr(ItB);
		if (DiffProperties(ValA, ValB, Property->ValueProp, Ctx))
		{
			Ctx.Diffs.Emplace(EDiffObjectNodeType::Property, A, B, Property, Idx);
			return true;
		}
	}
	return false;
}

bool DiffInterfaceProperties(const void* A, const void* B, const FInterfaceProperty* Property, FDiffObjectContext& Ctx)
{
	const FScriptInterface* InterfaceA = (FScriptInterface*)A;
	const FScriptInterface* InterfaceB = (FScriptInterface*)B;
	const UObject* ObjectA = InterfaceA->GetObject();
	const UObject* ObjectB = InterfaceB->GetObject();
	if (DiffObjectReferences(ObjectA, ObjectB, Ctx))
	{
		Ctx.Diffs.Emplace(EDiffObjectNodeType::Property, A, B, Property);
		return true;
	}
	return false;
}

bool DiffDelegateProperties(const void* A, const void* B, const FDelegateProperty* Property, FDiffObjectContext& Ctx)
{
	const FScriptDelegate* DelegateA = (const FScriptDelegate*)A;
	const FScriptDelegate* DelegateB = (const FScriptDelegate*)B;
	if (DelegateA->GetFunctionName() != DelegateB->GetFunctionName() ||
		DiffObjectReferences(DelegateA->GetUObject(), DelegateB->GetUObject(), Ctx))
	{
		Ctx.Diffs.Emplace(EDiffObjectNodeType::Property, A, B, Property);
		return true;
	}
	return false;
}

bool DiffFieldPathProperties(const void* A, const void* B, const FFieldPathProperty* Property, FDiffObjectContext& Ctx)
{
	using namespace FFieldPath_Private;
	const FFieldPath& ValueA = FFieldPathProperty::GetPropertyValue(A);
	const FFieldPath& ValueB = FFieldPathProperty::GetPropertyValue(B);
	if (ValueA.*_Path != ValueB.*_Path)
	{
		if (IgnoreDiff(A, B, Property, Ctx))
		{
			return false;
		}
		Ctx.Diffs.Emplace(EDiffObjectNodeType::Property, A, B, Property);
		return true;
	}

	const UObject* OwnerA = (ValueA.*_ResolvedOwner).Get();
	const UObject* OwnerB = (ValueB.*_ResolvedOwner).Get();
	if (DiffObjectReferences(OwnerA, OwnerB, Ctx))
	{
		Ctx.Diffs.Emplace(EDiffObjectNodeType::Property, A, B, Property);
		return true;
	}
	return false;
}

bool DiffProperties(const void* A, const void* B, const FProperty* Property, FDiffObjectContext& Ctx)
{
	uint64 Flags = Property->GetCastFlags();

	if (HasAnyCastClassFlags<CASTCLASS_FStructProperty>(Flags))
	{
		if (DiffStructProperties(A, B, static_cast<const FStructProperty*>(Property), Ctx))
		{
			return true;
		}
	}
	else if (HasAnyCastClassFlags<CASTCLASS_FObjectPropertyBase>(Flags))
	{
		if (DiffObjectProperties(A, B, static_cast<const FObjectPropertyBase*>(Property), Ctx))
		{
			return true;
		}
	}
	else if (HasAnyCastClassFlags<CASTCLASS_FArrayProperty>(Flags))
	{
		if (DiffArrayProperties(A, B, static_cast<const FArrayProperty*>(Property), Ctx))
		{
			return true;
		}
	}
	else if (HasAnyCastClassFlags<CASTCLASS_FSetProperty>(Flags))
	{
		if (DiffSetProperties(A, B, static_cast<const FSetProperty*>(Property), Ctx))
		{
			return true;
		}
	}
	else if (HasAnyCastClassFlags<CASTCLASS_FMapProperty>(Flags))
	{
		if (DiffMapProperties(A, B, static_cast<const FMapProperty*>(Property), Ctx))
		{
			return true;
		}
	}
	else if (HasAnyCastClassFlags<CASTCLASS_FInterfaceProperty>(Flags))
	{
		if (DiffInterfaceProperties(A, B, static_cast<const FInterfaceProperty*>(Property), Ctx))
		{
			return true;
		}
	}
	else if (HasAnyCastClassFlags<CASTCLASS_FDelegateProperty>(Flags))
	{
		if (DiffDelegateProperties(A, B, static_cast<const FDelegateProperty*>(Property), Ctx))
		{
			return true;
		}
	}
	else if (HasAnyCastClassFlags<CASTCLASS_FFieldPathProperty>(Flags))
	{
		if (DiffFieldPathProperties(A, B, static_cast<const FFieldPathProperty*>(Property), Ctx))
		{
			return true;
		}
	}
	else
	{
		if (!Property->Identical(A, B, PPF_DeepComparison))
		{
			if (IgnoreDiff(A, B, Property, Ctx))
			{
				return false;
			}
			Ctx.Diffs.Emplace(EDiffObjectNodeType::Property, A, B, Property);
			return true;
		}
	}
	return false;
}

bool DiffStructs(const void* A, const void* B, const UStruct* Struct, FDiffObjectContext& Ctx)
{
	const static FName ClassDefaultObject{"ClassDefaultObject"};
	for (FProperty* Property = Struct->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		if (!Property->ShouldDuplicateValue()) //Should the property be compared at all?
		{
			continue;
		}
		// don't compare the CDO property for classes to prevent infinite recursion
		if (Property->GetFName() == ClassDefaultObject && Struct->IsChildOf<UClass>())
		{
			continue;
		}
		const void* ValueA = Property->ContainerPtrToValuePtr<void>(A);
		const void* ValueB = Property->ContainerPtrToValuePtr<void>(B);
		if (DiffProperties(ValueA, ValueB, Property, Ctx))
		{
			return true;
		}
	}
	return false;
}

static bool EqualOverrideIds(const FOverriddenPropertyNodeID& A, const FOverriddenPropertyNodeID& B)
{
	using namespace FOverriddenPropertyNodeIdDiff_Private;
	const UObject* ObjectA = (A.*_Object).Get();
	const UObject* ObjectB = (B.*_Object).Get();

	if (ObjectA || ObjectB)
	{
		TStringBuilder<256> PathA;
		if (ObjectA)
		{
			GetRelativePathName(ObjectA, PathA);
		}
		
		TStringBuilder<256> PathB;
		if (ObjectB)
		{
			GetRelativePathName(ObjectB, PathB);
		}

		return PathA.ToView() == PathB.ToView();
	}
	else
	{
		return A == B;
	}
}

struct FDiffNodeContext
{
	FDiffNodeContext(const UObject* A)
	: ObjectA(A)
	, CurrentStruct(A->GetClass())
	{}

	const UObject* ObjectA = nullptr;
	UStruct* CurrentStruct = nullptr;
	FPropertyVisitorPath VisitorPath;
};

FDiffNodeContext MakeNewContext(const FOverriddenPropertyNode& SubNodeA, const FDiffNodeContext& Ctx)
{
	const UObject* SubA = Ctx.ObjectA;
	{
		using namespace FOverriddenPropertyNodeIdDiff_Private;
		SubA = (SubNodeA.GetNodeID().*_Object).Get();
	}
	FDiffNodeContext NewCtx(SubA ? SubA : Ctx.ObjectA);
	NewCtx.VisitorPath = Ctx.VisitorPath;
	FName PropertyName = GetOverridePropertyName(SubNodeA);
	FProperty* Prop = Ctx.CurrentStruct->FindPropertyByName(PropertyName);
	if (Prop)
	{
		FPropertyVisitorInfo PropInfo(Prop, Ctx.CurrentStruct);
		NewCtx.VisitorPath.Push(PropInfo);
		if (FStructProperty* StructProp = CastField<FStructProperty>(Prop))
		{
			NewCtx.CurrentStruct = StructProp->Struct;
		}
	}
	return NewCtx;
}

static bool DiffOverrideNodes(const FOverriddenPropertyNode& A, const FOverriddenPropertyNode& B, FDiffNodeContext& Ctx)
{
	EOverriddenPropertyOperation OpA = A.GetOperation();
	EOverriddenPropertyOperation OpB = B.GetOperation();

	if (EqualOverrideIds(A.GetNodeID(), B.GetNodeID()))
	{
		const TArray<FOverriddenPropertyNode>& SubNodesA = A.GetSubPropertyNodes();
		const TArray<FOverriddenPropertyNode>& SubNodesB = B.GetSubPropertyNodes();

		if (OpA == OpB)
		{
			if (SubNodesA.Num() == SubNodesB.Num())
			{
				int Index = 0;

				for (const FOverriddenPropertyNode& SubNodeA : SubNodesA)
				{
					// The sub-nodes might be in different order so find a match. This
					// could be optimized, however, the sub-node count is generally low.
					const FOverriddenPropertyNode* SubNodeB = nullptr;
					for (const FOverriddenPropertyNode& Test : SubNodesB)
					{
						if (EqualOverrideIds(Test.GetNodeID(), SubNodeA.GetNodeID()))
						{
							SubNodeB = &Test;
							break;
						}
					}

					if (SubNodeB)
					{
						FDiffNodeContext NewCtx = MakeNewContext(SubNodeA, Ctx);
						if (DiffOverrideNodes(SubNodeA, *SubNodeB, NewCtx))
						{
							return true;
						}
					}
					else
					{
						return true;
					}
				}
			}
			else
			{
				return true;
			}
		}
		// Don't consider semantically identical overridden state betwen 'all overridden modified' and 'replaced'
		// TPS doesn't reconstruct the state on load and hence needs to mark struct only modified in all overridden state
		// to handle schema changes
		else if (OpA == EOverriddenPropertyOperation::Modified && 
			OpB == EOverriddenPropertyOperation::Replace)
		{
			EOverriddenState State = FOverridableManager::Get().GetOverriddenState(
				const_cast<UObject*>(Ctx.ObjectA),
				{},												// consider all properties
				Ctx.VisitorPath,
				nullptr,
				true, 											// ignore added
				[](TNotNull<const UObject*>) { return false; }  // ignore subobjects
			);
			if (State != EOverriddenState::AllOverridden)
			{
				return true;
			}
		}
		else
		{
			return true;
		}
	}
	else
	{
		return true;
	}

	return false;
}

static bool DiffOverrides(const FOverriddenPropertySet* A, const FOverriddenPropertySet* B, FDiffNodeContext& Ctx)
{
	if (A && B)
	{
		return DiffOverrideNodes(A->GetRootOverriddenPropertyNode(), B->GetRootOverriddenPropertyNode(), Ctx);
	}
	
	// When both overrides are NULL then the object does not support
	// overridable serialization and should not be considered a diff.
	return A != B;
}

bool DiffObjects(FDiffObjectContext& Ctx)
{
	TMap<FString, const UObject*> ObjectsA;
	TMap<FString, const UObject*> ObjectsB;

	auto FillObjectMap = [](const UObject* Root, TMap<FString, const UObject*>& OutObjects)
	{
		OutObjects.Reserve(256);
		ForEachObjectWithOuter(Root, [Root,&OutObjects](UObject* Obj)
		{
			// const UObject* StopOuter = Obj->HasAnyFlags(RF_HasExternalPackage) ? Obj->GetOutermostObject()->GetOuter() : Root;
			OutObjects.Emplace(GetRelativePathName(Obj), Obj);
			return true;
		});
	};
	FillObjectMap(Ctx.RootA, ObjectsA);
	FillObjectMap(Ctx.RootB, ObjectsB);

	FOverridableManager& OverrideManager = FOverridableManager::Get();

	for (const TPair<FString, const UObject*>& PairA : ObjectsA)
	{
		const UObject* A = PairA.Value;
		const UObject** B = ObjectsB.Find(PairA.Key);
		if (!B)
		{
			Ctx.Diffs.Emplace(EDiffObjectNodeType::NotInB, A, nullptr, A->GetClass());
			return true;
		}

		Ctx.Visited.Reset();

		FDiffNodeContext NodeCtx(A);
		if (DiffOverrides(OverrideManager.GetOverriddenProperties(A), OverrideManager.GetOverriddenProperties(*B), NodeCtx))
		{
			Ctx.OverrideDiffs.Emplace(A, *B);
		}

		if (DiffObjects(A, *B, Ctx))
		{
			return true;
		}
	}

	for (const TPair<FString, const UObject*>& PairB : ObjectsB)
	{
		const UObject* B = PairB.Value;
		const UObject** A = ObjectsA.Find(PairB.Key);
		if (!A)
		{
			Ctx.Diffs.Emplace(EDiffObjectNodeType::NotInA, nullptr, B, B->GetClass());
			return true;
		}
	}
	return false;
}

///////////////////////////////////////////////////////////////////////////////

void PrintDiff(FUtf8StringBuilderBase& Out, FDiffObjectContext& Ctx)
{
	check(Ctx.Diffs.Num());
	const FDiffObjectNode& LeafNode = Ctx.Diffs[0];
	const FDiffObjectNode& RootNode = Ctx.Diffs.Last();

	// Build DiffType:('ValueA' != 'ValueB') string from the first/innermost node
	if (LeafNode.Type == EDiffObjectNodeType::Property)
	{
		const FProperty* Property = LeafNode.GetProperty();
		FString A;
		FString B;
		Property->ExportTextItem_Direct(A, LeafNode.A, nullptr, (UObject*)Ctx.RootA, PPF_None, nullptr);
		Property->ExportTextItem_Direct(B, LeafNode.B, nullptr, (UObject*)Ctx.RootB, PPF_None, nullptr);
		Out << Property->GetClass()->GetFName();
		Out << ": ('";
		A.ReplaceInline(TEXT("\r\n"), TEXT(" "));
		B.ReplaceInline(TEXT("\r\n"), TEXT(" "));
		Out << A << "' != '" << B;
	}
	else
	{
		// Append diff type
		Out << ToString(LeafNode.Type);
		Out << ": ('";
		const UObject* A = static_cast<const UObject*>(LeafNode.A);
		const UObject* B = static_cast<const UObject*>(LeafNode.B);
		Out << (A ? A->GetFullName() : "<null>");
		Out << "' != '";
		Out << (B ? B->GetFullName() : "<null>");
	}
	Out << "') in ";

	// Build path root elemeent from the last/outermost UObject
	check(RootNode.Type != EDiffObjectNodeType::Property);
	if (&LeafNode != &RootNode)
	{
		const UObject* A = static_cast<const UObject*>(RootNode.A);
		const UObject* B = static_cast<const UObject*>(RootNode.B);
		if (const UClass* Class = RootNode.GetClass())
		{
			Out << (A ? A->GetName() : B->GetName());
			Out << ':';
			Out << Class->GetPrefixCPP() << Class->GetName();
		}
	}

	// Build path chain based on all property nodes
	bool bPrintPropertyPath = true;
	for (const FDiffObjectNode& Diff : ReverseIterate(Ctx.Diffs))
	{
		if (Diff.Type != EDiffObjectNodeType::Property && Diff.Type != EDiffObjectNodeType::Class)
		{
			// all intermediate diff nodes should be properties
			check(&Diff == &LeafNode || &Diff == &RootNode);
			continue;
		}
		// Append .CPPMemberName:CPPMemberType to path string
		// skip for inner properties coming after a range property, they just repeat the same type info again
		if (bPrintPropertyPath)
		{
			if (Diff.Type == EDiffObjectNodeType::Property)
			{
				const FProperty* Property = Diff.GetProperty();
				FString ExtendedTypeText;
				FString TypeText = Property->GetCPPType(&ExtendedTypeText);
				Out << '.';
				Out << Property->GetFName();
				Out << ':';
				Out << TypeText << ExtendedTypeText;
			}
			else
			{
				check(Diff.Type == EDiffObjectNodeType::Class);
				Out << ".Class:";
				Out << Diff.GetClass()->GetFName();
			}
		}
		// Append range index
		if (Diff.Idx != ~0ull)
		{
			Out << "[" << Diff.Idx << "]";
			bPrintPropertyPath = false;
		}
		else
		{
			bPrintPropertyPath = true;
		}
	}
}

} // namespace PlainProps::UE
