// Copyright Epic Games, Inc. All Rights Reserved.

// Temporary custom UObject diffing code that allows for comparing two object graphs by treating
// internal object references as equal if their relative object paths are equal.
// This is currently only used by RoundtripViaLinkerBatch which will be updated to use a more robust
// and scalable approach that also captures custom bound data.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Templates/Function.h"
#include "UObject/NameTypes.h"

class FProperty;
class UClass;
class UObject;

namespace PlainProps::UE
{

enum class EDiffObjectNodeType : uint8
{
	NotInA,		// An object under RootB is missing in RootA
	NotInB,		// An object under RootA is missing in RootB
	Class,		// Two object pointers A and B are different if:
				// - both are set but the UClass is different, i.e. A->GetClass() != B->GetClass()
	Object,		// Two object pointers A and B are different if:
				// - only one of them is null, i.e. (A==nullptr != B==nullptr)
				// - only one of them is an internal reference, i.e. (A->IsIn(RootA) != B->IsIn(RootB))
				// - both are internal references but the relative paths inside their roots are different
				// - both are external references and they have member properties with different values.
	Property	// Two properties A and B have different values
};

struct FDiffObjectNode
{
	EDiffObjectNodeType Type;
	const void* A;
	const void* B;
	const void* TypePtr = nullptr;
	uint64 Idx = ~0ull;

	const FProperty* GetProperty() const
	{
		return Type == EDiffObjectNodeType::Property ? (FProperty*)TypePtr : nullptr;
	}

	const UClass* GetClass() const
	{
		return Type != EDiffObjectNodeType::Property ? (UClass*)TypePtr : nullptr;
	}
};

struct FDiffOverride
{
	const UObject* A;
	const UObject* B;
};

struct FDiffObjectFilter
{
	// Structs for which to bypass the native identical function
	TSet<FName> BypassNativeIdenticalStructs;
	// Structs for which to ignore any diffs
	TSet<FName> IgnoreStructs;
	// Struct specific properties for which to ignore any diffs
	TSet<TPair<FName, FName>> IgnorePropertiesForStructs;
	// Base class properties for which to ignore any diffs (more expensive to check)
	TMap<FName, FName> IgnorePropertiesForBases;
	// Cast flags for properties for which to ignore any diffs
	uint64 IgnoreCastFlags = 0;//CASTCLASS_FMulticastInlineDelegateProperty|CASTCLASS_FMulticastSparseDelegateProperty;
};

struct FDiffObjectContext
{
	// in
	const UObject* const RootA = nullptr;
	const UObject* const RootB = nullptr;
	const FDiffObjectFilter& Filter;

	// temporary data structures	
	TSet<TPair<const UObject*, const UObject*>> Visited;

	// out
	TArray<FDiffObjectNode> Diffs;
	TArray<FDiffOverride> OverrideDiffs;
	uint32 NumIgnoredDiffs = 0;
};

bool DiffObjects(FDiffObjectContext& Ctx);

void PrintDiff(FUtf8StringBuilderBase& Out, FDiffObjectContext& Ctx);

}
