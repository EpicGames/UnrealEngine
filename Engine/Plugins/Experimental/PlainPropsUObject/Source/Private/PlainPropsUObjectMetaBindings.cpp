// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlainPropsUObjectMetaBindingsInternal.h"
#include "PlainPropsInternalPrivateMemberPtr.h"
#include "PlainPropsUObjectBindingsInternal.h"
#include "Misc/DefinePrivateMemberPtr.h"
#include "UObject/Class.h"
#include "UObject/CoreNetTypes.h"
#include "UObject/LinkerLoad.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"
#include "UObject/EnumProperty.h"
#include "UObject/PropertyOptional.h"

// Temp hacks. Long-term either add FProperty getters for ctor/dtor/hash function pointers 
// and delegate APIs for non-intrusive serialization or integrate PlainProps into Core/CoreUObject

UE_DEFINE_PRIVATE_MEMBER_PTR(TObjectPtr<UStruct>, GSuperStruct, UStruct, SuperStruct);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
PP_DEFINE_PRIVATE_MEMBER_NAMESPACE_BEGIN(UClass_Private)
PP_DEFINE_PRIVATE_MEMBER_PTR(UClass, ClassDefaultObject, TObjectPtr<UObject>);
#if PP_OVERRIDE_OBJECT_LOADING
#if USTRUCT_FAST_ISCHILDOF_IMPL == USTRUCT_ISCHILDOF_STRUCTARRAY
PP_DEFINE_PRIVATE_MEMBER_PTR(UStruct, ReinitializeBaseChainArray, void());
#endif
#endif
PP_DEFINE_PRIVATE_MEMBER_NAMESPACE_END()

PP_DEFINE_PRIVATE_MEMBER_NAMESPACE_BEGIN(FProperty_Private)
PP_DEFINE_PRIVATE_MEMBER_PTR(FProperty, ElementSize,					int32);
PP_DEFINE_PRIVATE_MEMBER_PTR(FProperty, BlueprintReplicationCondition,	TEnumAsByte<ELifetimeCondition>);
PP_DEFINE_PRIVATE_MEMBER_PTR(FProperty, Offset_Internal,				int32);
PP_DEFINE_PRIVATE_MEMBER_NAMESPACE_END()

PP_DEFINE_PRIVATE_MEMBER_NAMESPACE_BEGIN(FBoolProperty_Private)
PP_DEFINE_PRIVATE_MEMBER_PTR(FBoolProperty, FieldSize,	uint8);
PP_DEFINE_PRIVATE_MEMBER_PTR(FBoolProperty, ByteOffset,	uint8);
PP_DEFINE_PRIVATE_MEMBER_PTR(FBoolProperty, ByteMask,	uint8);
PP_DEFINE_PRIVATE_MEMBER_PTR(FBoolProperty, FieldMask,	uint8);
PP_DEFINE_PRIVATE_MEMBER_NAMESPACE_END()

PP_DEFINE_PRIVATE_MEMBER_NAMESPACE_BEGIN(FEnumProperty_Private)
PP_DEFINE_PRIVATE_MEMBER_PTR(FEnumProperty, UnderlyingProp, FNumericProperty*);
PP_DEFINE_PRIVATE_MEMBER_PTR(FEnumProperty, Enum, TObjectPtr<UEnum>);
PP_DEFINE_PRIVATE_MEMBER_NAMESPACE_END()

PRAGMA_ENABLE_DEPRECATION_WARNINGS

namespace PlainProps::UE
{

////////////////////////////////////////////////////////////////////////////////////////////////
// Native FProperty meta bindings

static void SaveSingleProperty(FMemberBuilder& Dst, const FMemberId& TypeNameMemberId, const FMemberId& PropertyMemberId, FProperty* InProperty, const FSaveContext& Ctx)
{
	const FName PropertyTypeName = InProperty ? InProperty->GetClass()->GetFName() : FName();
	Dst.AddStruct(TypeNameMemberId, GUE.Structs.Name, SaveStruct(&PropertyTypeName, GUE.Structs.Name, Ctx));

	if (InProperty)
	{
		FBindId Id = IndexPropertyMeta(InProperty);
		Dst.AddStruct(PropertyMemberId, Id, SaveStruct(InProperty, Id, Ctx));
	}
}

static void LoadSingleProperty(FMemberLoader& Members, FProperty*& Dst, const FProperty& Owner)
{
	FName PropertyTypeName;
	LoadStruct(&PropertyTypeName, Members.GrabStruct());

	if (!PropertyTypeName.IsNone())
	{
		Dst = CastField<FProperty>(FField::Construct(PropertyTypeName, &Owner, FName()));
		FBindId Id = IndexPropertyMeta(Dst);
		LoadStruct(Dst, Members.GrabStruct());
	}
}

FPropertyBinding::FPropertyBinding(TPropertySpecifier<NumMembers>& Spec)
	: PropertyFlagsId(GetEnumId<Ids, EPropertyFlags>())
	, LifetimeConditionId(Ids::IndexEnum(GUE.Scopes.CoreUObject, "ELifetimeCondition")) // too early for GUE.DynamicIds.GetEnum
{
	// All FFields are FProperties so we can fold all FField members here for simplicity and efficiency
	Spec.Members[M::NamePrivate] = GUE.Structs.Name;
	Spec.Members[M::ArrayDim] = Specify<int32>();
	Spec.Members[M::ElementSize] = Specify<int32>();
	Spec.Members[M::PropertyFlags] = Specify<EPropertyFlags>(PropertyFlagsId);
	Spec.Members[M::RepIndex] = Specify<uint16>();
	// Todo: Cleanup TEnumAsByte reflection, for now specify/save/load the same enum8 type as the schema version, see FPropertyBinder::BindByte
	Spec.Members[M::BlueprintReplicationCondition] = FMemberSpec(ELeafType::Enum, ELeafWidth::B8, LifetimeConditionId);
	// Todo: Save and load editor only FField::MetaDataMap
	// TMap<FName, FString>* MetaDataMap;
}

void FPropertyBinding::Save(FMemberBuilder& Dst, const FProperty& Src, const FProperty*, const FSaveContext& Ctx) const
{
	using namespace FProperty_Private;
	Dst.AddStruct(MemberIds[M::NamePrivate], GUE.Structs.Name, SaveStruct(&Src.NamePrivate, GUE.Structs.Name, Ctx));
	Dst.Add(MemberIds[M::ArrayDim], Src.ArrayDim);
	Dst.Add(MemberIds[M::ElementSize], Src.*_ElementSize);
	EPropertyFlags SaveFlags = Src.PropertyFlags & ~CPF_ComputedFlags;
	Dst.AddEnum(MemberIds[M::PropertyFlags], PropertyFlagsId, SaveFlags);
	Dst.Add(MemberIds[M::RepIndex], Src.RepIndex);
	Dst.AddEnum<uint8>(MemberIds[M::BlueprintReplicationCondition], LifetimeConditionId, (Src.*_BlueprintReplicationCondition).GetIntValue());
	// TMap<FName, FString>* MetaDataMap;
}

void FPropertyBinding::Load(FProperty& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
{
	using namespace FProperty_Private;
	FMemberLoader Members(Src);
	LoadStruct(&Dst.NamePrivate, Members.GrabStruct());
	Dst.ArrayDim = Members.GrabLeaf().As<int32>();
	Dst.*_ElementSize = Members.GrabLeaf().As<int32>();
	EPropertyFlags LoadFlags = Members.GrabLeaf().As<EPropertyFlags>();
	Dst.PropertyFlags = (LoadFlags & ~CPF_ComputedFlags) | (Dst.PropertyFlags & CPF_ComputedFlags);
	Dst.RepIndex = Members.GrabLeaf().As<uint16>();
	Dst.*_BlueprintReplicationCondition = TEnumAsByte<ELifetimeCondition>(Members.GrabLeaf().AsUnderlyingValue<uint8>());
	// TMap<FName, FString>* MetaDataMap;
	Dst.*_Offset_Internal = 0;
	Dst.DestructorLinkNext = nullptr;
	check(!Members.HasMore());
}

FBoolPropertyBinding::FBoolPropertyBinding(TPropertySpecifier<NumMembers>& Spec)
{
	Spec.Super = GUE.Properties.Property;
	Spec.Members[M::FieldSize] = Specify<uint8>();
	Spec.Members[M::ByteOffset] = Specify<uint8>();
	Spec.Members[M::ByteMask] = Specify<uint8>();
	Spec.Members[M::FieldMask] = Specify<uint8>();
}

void FBoolPropertyBinding::Save(FMemberBuilder& Dst, const FBoolProperty& Src, const FBoolProperty*, const FSaveContext& Ctx) const
{
	using namespace FBoolProperty_Private;
	SaveSuper(Dst, &Src, GUE.Properties.Property, Ctx);
	Dst.Add(MemberIds[M::FieldSize], Src.*_FieldSize);
	Dst.Add(MemberIds[M::ByteOffset], Src.*_ByteOffset);
	Dst.Add(MemberIds[M::ByteMask], Src.*_ByteMask);
	Dst.Add(MemberIds[M::FieldMask], Src.*_FieldMask);
}

void FBoolPropertyBinding::Load(FBoolProperty& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
{
	using namespace FBoolProperty_Private;
	FMemberLoader Members(Src);
	TryLoadSuper(&Dst, Members);
	Dst.*_FieldSize = Members.GrabLeaf().As<uint8>();
	Dst.*_ByteOffset = Members.GrabLeaf().As<uint8>();
	Dst.*_ByteMask = Members.GrabLeaf().As<uint8>();
	Dst.*_FieldMask = Members.GrabLeaf().As<uint8>();
	check(!Members.HasMore());
}

FStructPropertyBinding::FStructPropertyBinding(TPropertySpecifier<NumMembers>& Spec)
{
	Spec.Super = GUE.Properties.Property;
	Spec.Members[M::Struct] = GUE.Structs.ObjectPtr;
}

void FStructPropertyBinding::Save(FMemberBuilder& Dst, const FStructProperty& Src, const FStructProperty*, const FSaveContext& Ctx) const
{
	SaveSuper(Dst, &Src, GUE.Properties.Property, Ctx);
	Dst.AddStruct(MemberIds[M::Struct], GUE.Structs.ObjectPtr, SaveStruct(&Src.Struct, GUE.Structs.ObjectPtr, Ctx));
}

void FStructPropertyBinding::Load(FStructProperty& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
{
	FMemberLoader Members(Src);
	TryLoadSuper(&Dst, Members);
	LoadStruct(&Dst.Struct, Members.GrabStruct());
	check(!Members.HasMore());
}

FClassPropertyBinding::FClassPropertyBinding(TPropertySpecifier<NumMembers>& Spec)
{
	Spec.Super = GUE.Properties.Property;
	Spec.Members[M::MetaClass] = GUE.Structs.ObjectPtr;
}

void FClassPropertyBinding::Save(FMemberBuilder& Dst, const FClassProperty& Src, const FClassProperty*, const FSaveContext& Ctx) const
{
	SaveSuper(Dst, &Src, GUE.Properties.Property, Ctx);
	Dst.AddStruct(MemberIds[M::MetaClass], GUE.Structs.ObjectPtr, SaveStruct(&Src.MetaClass, GUE.Structs.ObjectPtr, Ctx));
}

void FClassPropertyBinding::Load(FClassProperty& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
{
	FMemberLoader Members(Src);
	TryLoadSuper(&Dst, Members);
	LoadStruct(&Dst.MetaClass, Members.GrabStruct());
	// Todo: AddReferencingProperty requires private header "UObject/LinkerPlaceholderClass.h"
	// #if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	// 	if (ULinkerPlaceholderClass* PlaceholderClass = Cast<ULinkerPlaceholderClass>(Dst.MetaClass))
	// 	{
	// 		PlaceholderClass->AddReferencingProperty(&Dst);
	// 	}
	// #endif
	check(!Members.HasMore());
}

FObjectPropertyBaseBinding::FObjectPropertyBaseBinding(TPropertySpecifier<NumMembers>& Spec)
{
	Spec.Super = GUE.Properties.Property;
	Spec.Members[M::PropertyClass] = GUE.Structs.ObjectPtr;
}

void FObjectPropertyBaseBinding::Save(FMemberBuilder& Dst, const FObjectPropertyBase& Src, const FObjectPropertyBase*, const FSaveContext& Ctx) const
{
	SaveSuper(Dst, &Src, GUE.Properties.Property, Ctx);
	Dst.AddStruct(MemberIds[M::PropertyClass], GUE.Structs.ObjectPtr, SaveStruct(&Src.PropertyClass, GUE.Structs.ObjectPtr, Ctx));
}

void FObjectPropertyBaseBinding::Load(FObjectPropertyBase& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
{
	FMemberLoader Members(Src);
	TryLoadSuper(&Dst, Members);
	LoadStruct(&Dst.PropertyClass, Members.GrabStruct());
	check(!Members.HasMore());
}

FArrayPropertyBinding::FArrayPropertyBinding(TPropertySpecifier<NumMembers>& Spec)
{
	Spec.Super = GUE.Properties.Property;
	// Intentionally skip ArrayFlags (no support for blueprint UsesMemoryImageAllocator)
	Spec.Members[M::InnerPropertyTypeName] = GUE.Structs.Name;
	Spec.Members[M::InnerProperty] = SpecDynamicStruct;
}

void FArrayPropertyBinding::Save(FMemberBuilder& Dst, const FArrayProperty& Src, const FArrayProperty*, const FSaveContext& Ctx) const
{
	SaveSuper(Dst, &Src, GUE.Properties.Property, Ctx);
	
	SaveSingleProperty(Dst, MemberIds[M::InnerPropertyTypeName], MemberIds[M::InnerProperty], Src.Inner, Ctx);
}

void FArrayPropertyBinding::Load(FArrayProperty& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
{
	FMemberLoader Members(Src);
	TryLoadSuper(&Dst, Members);
	
	LoadSingleProperty(Members, Dst.Inner, Dst);

	check(!Members.HasMore());
}

FSetPropertyBinding::FSetPropertyBinding(TPropertySpecifier<NumMembers>& Spec)
{
	Spec.Super = GUE.Properties.Property;
	Spec.Members[M::InnerPropertyTypeName] = GUE.Structs.Name;
	Spec.Members[M::InnerProperty] = SpecDynamicStruct;
}

void FSetPropertyBinding::Save(FMemberBuilder& Dst, const FSetProperty& Src, const FSetProperty*, const FSaveContext& Ctx) const
{
	SaveSuper(Dst, &Src, GUE.Properties.Property, Ctx);

	SaveSingleProperty(Dst, MemberIds[M::InnerPropertyTypeName], MemberIds[M::InnerProperty], Src.ElementProp, Ctx);
}

void FSetPropertyBinding::Load(FSetProperty& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
{
	FMemberLoader Members(Src);
	TryLoadSuper(&Dst, Members);

	LoadSingleProperty(Members, Dst.ElementProp, Dst);

	check(!Members.HasMore());
}

FMapPropertyBinding::FMapPropertyBinding(TPropertySpecifier<NumMembers>& Spec)
{
	Spec.Super = GUE.Properties.Property;
	Spec.Members[M::KeyPropertyTypeName] = GUE.Structs.Name;
	Spec.Members[M::KeyProperty] = SpecDynamicStruct;
	Spec.Members[M::ValuePropertyTypeName] = GUE.Structs.Name;
	Spec.Members[M::ValueProperty] = SpecDynamicStruct;
}

void FMapPropertyBinding::Save(FMemberBuilder& Dst, const FMapProperty& Src, const FMapProperty*, const FSaveContext& Ctx) const
{
	SaveSuper(Dst, &Src, GUE.Properties.Property, Ctx);

	SaveSingleProperty(Dst, MemberIds[M::KeyPropertyTypeName], MemberIds[M::KeyProperty], Src.KeyProp, Ctx);
	SaveSingleProperty(Dst, MemberIds[M::ValuePropertyTypeName], MemberIds[M::ValueProperty], Src.ValueProp, Ctx);
}

void FMapPropertyBinding::Load(FMapProperty& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
{
	FMemberLoader Members(Src);
	TryLoadSuper(&Dst, Members);

	LoadSingleProperty(Members, Dst.KeyProp, Dst);
	LoadSingleProperty(Members, Dst.ValueProp, Dst);

	check(!Members.HasMore());
}

FOptionalPropertyBinding::FOptionalPropertyBinding(TPropertySpecifier<NumMembers>& Spec)
{
	Spec.Super = GUE.Properties.Property;
	Spec.Members[M::ValuePropertyTypeName] = GUE.Structs.Name;
	Spec.Members[M::ValueProperty] = SpecDynamicStruct;
}

void FOptionalPropertyBinding::Save(FMemberBuilder& Dst, const FOptionalProperty& Src, const FOptionalProperty* Default, const FSaveContext& Ctx) const
{
	SaveSuper(Dst, &Src, GUE.Properties.Property, Ctx);
	SaveSingleProperty(Dst, MemberIds[M::ValuePropertyTypeName], MemberIds[M::ValueProperty], Src.GetValueProperty(), Ctx);
}

void FOptionalPropertyBinding::Load(FOptionalProperty& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
{
	FMemberLoader Members(Src);
	TryLoadSuper(&Dst, Members);

	FProperty* ValueProperty = nullptr;
	LoadSingleProperty(Members, ValueProperty, Dst);
	Dst.SetValueProperty(ValueProperty);

	check(!Members.HasMore());
}

FBytePropertyBinding::FBytePropertyBinding(TPropertySpecifier<NumMembers>& Spec)
{
	Spec.Super = GUE.Properties.Property;
	Spec.Members[M::Enum] = GUE.Structs.ObjectPtr;
}

void FBytePropertyBinding::Save(FMemberBuilder& Dst, const FByteProperty& Src, const FByteProperty* Default, const FSaveContext& Ctx) const
{
	SaveSuper(Dst, &Src, GUE.Properties.Property, Ctx);
	Dst.AddStruct(MemberIds[M::Enum], GUE.Structs.ObjectPtr, SaveStruct(&Src.Enum, GUE.Structs.ObjectPtr, Ctx));
}

void FBytePropertyBinding::Load(FByteProperty& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
{
	FMemberLoader Members(Src);
	TryLoadSuper(&Dst, Members);
	LoadStruct(&Dst.Enum, Members.GrabStruct());
	check(!Members.HasMore());
}

FEnumPropertyBinding::FEnumPropertyBinding(TPropertySpecifier<NumMembers>& Spec)
{
	Spec.Super = GUE.Properties.Property;
	Spec.Members[M::UnderlyingPropertyTypeName] = GUE.Structs.Name;
	Spec.Members[M::UnderlyingProperty] = SpecDynamicStruct;
	Spec.Members[M::Enum] = GUE.Structs.ObjectPtr;
}

void FEnumPropertyBinding::Save(FMemberBuilder& Dst, const FEnumProperty& Src, const FEnumProperty* Default, const FSaveContext& Ctx) const
{
	using namespace FEnumProperty_Private;
	SaveSuper(Dst, &Src, GUE.Properties.Property, Ctx);

	SaveSingleProperty(Dst, MemberIds[M::UnderlyingPropertyTypeName], MemberIds[M::UnderlyingProperty], Src.*_UnderlyingProp, Ctx);

	Dst.AddStruct(MemberIds[M::Enum], GUE.Structs.ObjectPtr, SaveStruct(&(Src.*_Enum), GUE.Structs.ObjectPtr, Ctx));
}

void FEnumPropertyBinding::Load(FEnumProperty& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
{
	using namespace FEnumProperty_Private;
	FMemberLoader Members(Src);
	TryLoadSuper(&Dst, Members);
	
	FProperty* UnderlyingProp = nullptr;
	LoadSingleProperty(Members, UnderlyingProp, Dst);
	Dst.*_UnderlyingProp = CastField<FNumericProperty>(UnderlyingProp);

	LoadStruct(&(Dst.*_Enum), Members.GrabStruct());
	check(!Members.HasMore());
}

////////////////////////////////////////////////////////////////////////////////////////////////

UStructBinding::UStructBinding(TPropertySpecifier<NumMembers>& Spec)
{
	Spec.Members[M::SuperStruct] = GUE.Structs.ObjectPtr;
	// Children is an array of UFields, these are always UFunctions
	Spec.Members[M::Children] = DefaultRangeOf(GUE.Structs.ObjectPtr);
	// This UStruct base binding saves and loads the linked list of FProperties as an array.
	// The derived meta classes saves and loads the actual FProperty data as named schema members.
	Spec.Members[M::ChildPropertyClasses] = DefaultRangeOf(GUE.Structs.Name);
}

FTypedRange UStructBinding::SaveChildrenStatic(const UField* Children, int32 Num, const FSaveContext& Ctx)
{
	FBindId Id = GUE.Structs.ObjectPtr;
	FStructRangeSaver Out(Ctx.Scratch, Num);
	for (const UField* Child = Children; Child; Child = Child->Next)
	{
		Out.AddItem(SaveStruct(&Child, Id, Ctx));
	}
	return Out.Finalize(MakeStructRangeSchema(DefaultRangeMax, Id));
}

FTypedRange UStructBinding::SaveChildPropertyClassesStatic(const FField* ChildProperties, int32 Num, const FSaveContext& Ctx)
{
	FBindId Id = GUE.Structs.Name;
	FStructRangeSaver Out(Ctx.Scratch, Num);
	for (const FField* Child = ChildProperties; Child; Child = Child->Next)
	{
		FName ClassName = Child->GetClass()->GetFName();
		Out.AddItem(SaveStruct(&ClassName, Id, Ctx));
	}
	return Out.Finalize(MakeStructRangeSchema(DefaultRangeMax, Id));
}

void UStructBinding::Save(FMemberBuilder& Dst, const UStruct& Src, const UStruct*, const FSaveContext& Ctx) const
{
	Dst.AddStruct(MemberIds[M::SuperStruct], GUE.Structs.ObjectPtr, SaveStruct(&(Src.*GSuperStruct), GUE.Structs.ObjectPtr, Ctx));
	if (Src.Children)
	{
		int32 Num = 0;
		for (const UField* Child = Src.Children; Child; Child = Child->Next, ++Num);

		Dst.AddRange(MemberIds[M::Children], SaveChildrenStatic(Src.Children, Num, Ctx));
	}
	if (Src.ChildProperties)
	{
		int32 Num = 0;
		for (const FField* Child = Src.ChildProperties; Child; Child = Child->Next, ++Num);

		Dst.AddRange(MemberIds[M::ChildPropertyClasses], SaveChildPropertyClassesStatic(Src.ChildProperties, Num, Ctx));
	}
}

void UStructBinding::LoadChildrenStatic(TObjectPtr<UField>& Children, FStructRangeLoadView Src)
{
	Children = nullptr;
	UField* Previous = nullptr;
	for (FStructLoadView SrcIt : Src)
	{
		TObjectPtr<UField> Current;
		LoadStruct(&Current, SrcIt);
		if (Current)
		{
			if (!Children)
			{
				Children = Current;
			}
			if (Previous)
			{
				Previous->Next = Current;
			}
			Previous = Current;
		}
	}
	if (Previous)
	{
		Previous->Next = nullptr;
	}
}

void UStructBinding::LoadChildPropertyClassesStatic(UStruct& Dst, FStructRangeLoadView Src)
{
	Dst.ChildProperties = nullptr;
	FField* Previous = nullptr;
	for (FStructLoadView SrcIt : Src)
	{
		FName PropertyTypeName;
		LoadStruct(&PropertyTypeName, SrcIt);
		FField* Current = FField::Construct(PropertyTypeName, &Dst, NAME_None);
		check(Current);
		if (!Dst.ChildProperties)
		{
			Dst.ChildProperties = Current;
		}
		if (Previous)
		{
			Previous->Next = Current;
		}
		Previous = Current;
	}
	if (Previous)
	{
		Previous->Next = nullptr;
	}
}

void UStructBinding::Load(UStruct& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
{
	using namespace UClass_Private;

	FMemberLoader Members(Src);
	LoadStruct(&(Dst.*GSuperStruct), Members.GrabStruct());
#if PP_OVERRIDE_OBJECT_LOADING
#if USTRUCT_FAST_ISCHILDOF_IMPL == USTRUCT_ISCHILDOF_STRUCTARRAY
	(Dst.*_ReinitializeBaseChainArray)();
#endif
#endif
	if (Members.HasMore())
	{
		check(Members.PeekKind() == EMemberKind::Range);
		if (Members.PeekName() == MemberIds[M::Children])
		{
			LoadChildrenStatic(Dst.Children, Members.GrabRange().AsStructs());
		}
	}
	if (Members.HasMore())
	{
		check(Members.PeekKind() == EMemberKind::Range);
		check(Members.PeekName() == MemberIds[M::ChildPropertyClasses]);
		LoadChildPropertyClassesStatic(Dst, Members.GrabRange().AsStructs());
	}
	// UStruct::Serialize first serializes all FProperties, then calls link for all types but UClasses.
	// This base custom binding does not load the FProperties (only their types for constructing them),
	// so it always delegates the responsibility to call Link to the derived meta classes.
	check(!Members.HasMore());
}

UClassBinding::UClassBinding(TPropertySpecifier<NumMembers>& Spec)
: ClassFlagsId(GetEnumId<Ids, EClassFlags>())
, ImplementedInterfaceId(GetStructDualId<Ids, FImplementedInterface>())
{
	Spec.Super = GUE.Structs.Struct;
	Spec.Members[M::ClassFlags] = Specify<EClassFlags>(ClassFlagsId);
	Spec.Members[M::ClassWithin] = GUE.Structs.ObjectPtr;
	Spec.Members[M::ClassConfigName] = GUE.Structs.Name;
#if WITH_EDITORONLY_DATA
	Spec.Members[M::ClassGeneratedBy] = GUE.Structs.ObjectPtr;
#endif
	Spec.Members[M::Interfaces] = DefaultRangeOf(ImplementedInterfaceId);
}

FTypedRange UClassBinding::SaveInterfacesStatic(FBindId Id, TConstArrayView<FImplementedInterface> Interfaces, const FSaveContext& Ctx)
{
	FStructRangeSaver Out(Ctx.Scratch, Interfaces.Num());
	for (const FImplementedInterface& Interface : Interfaces)
	{
		Out.AddItem(SaveStruct(&Interface, Id, Ctx));
	}
	return Out.Finalize(MakeStructRangeSchema(DefaultRangeMax, Id));
}

void UClassBinding::LoadInterfacesStatic(TArray<FImplementedInterface>& Interfaces, FStructRangeLoadView Src)
{
	Interfaces.SetNum(static_cast<int32>(Src.Num()));
	FImplementedInterface* DstIt = Interfaces.GetData();
	for (FStructLoadView SrcIt : Src)
	{
		LoadStruct(DstIt++, SrcIt);
	}
}

void UClassBinding::Save(FMemberBuilder& Dst, const UClass& Src, const UClass*, const FSaveContext& Ctx) const
{
	SaveSuper(Dst, &Src, GUE.Structs.Struct, Ctx);
	EClassFlags SavedClassFlags = Src.ClassFlags & ~CLASS_ShouldNeverBeLoaded;
	Dst.AddEnum(MemberIds[M::ClassFlags], ClassFlagsId, SavedClassFlags);
	Dst.AddStruct(MemberIds[M::ClassWithin], GUE.Structs.ObjectPtr, SaveStruct(&Src.ClassWithin, GUE.Structs.ObjectPtr, Ctx));
	Dst.AddStruct(MemberIds[M::ClassConfigName], GUE.Structs.Name, SaveStruct(&Src.ClassConfigName, GUE.Structs.Name, Ctx));
#if WITH_EDITORONLY_DATA
	Dst.AddStruct(MemberIds[M::ClassGeneratedBy], GUE.Structs.ObjectPtr, SaveStruct(&Src.ClassGeneratedBy, GUE.Structs.ObjectPtr, Ctx));
#endif
	if (Src.Interfaces.Num())
	{
		Dst.AddRange(MemberIds[M::Interfaces], SaveInterfacesStatic(ImplementedInterfaceId, Src.Interfaces, Ctx));
	}
}

void UClassBinding::Load(UClass& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
{
	FMemberLoader Members(Src);

#if PP_OVERRIDE_OBJECT_LOADING
	UnhashObject(&Dst);
	TryLoadSuper(&Dst, Members);
	HashObject(&Dst);
#endif

	// UClass::Serialize calls Super::Serialize here to serialize all FProperties without yet calling Link

	EClassFlags LoadedClassFlags = Members.GrabLeaf().As<EClassFlags>();
	Dst.ClassFlags = LoadedClassFlags & ~CLASS_ShouldNeverBeLoaded;
	LoadStruct(&Dst.ClassWithin, Members.GrabStruct());
	LoadStruct(&Dst.ClassConfigName, Members.GrabStruct());
#if WITH_EDITORONLY_DATA
	LoadStruct(&Dst.ClassGeneratedBy, Members.GrabStruct());
#endif

	// Generate the FuncMap on load, it is only depending on the already loaded UFunction children,
	// and it is not dependent on the not-yet-loaded FProperties.
	// In the legacy system the FuncMap is explicitly serialized before the class flags.
	for (UField* Child = Dst.Children; Child; Child = Child->Next)
	{
		UFunction* Function = CastChecked<UFunction>(Child);
		Dst.AddFunctionToFunctionMap(Function, Function->GetFName());
	}

	// UClass::Serialize calls Link here AFTER serializing all FProperties and class flags,
	// but BEFORE serializing the Interfaces and ClassDefaultObject.
	// This base custom binding does not load the FProperties (only their types for constructing them),
	// so it always delegates the responsibility to call Link to the derived meta classes.

	// Don't save and load ClassDefaultObject from this base custom binding,
	// it must be loaded explicitly after Link has been called, which happens right after
	// loading FProperties in the meta binding that derives from UClass.

	// Todo: Since VER_UE4_UCLASS_SERIALIZE_INTERFACES_AFTER_LINKING UClass::Serialize very explicitly 
	// serializes the interfaces AFTER calling Link.
	// Loading them here in the base custom binding will take place before the derived meta binding calls Link!!!
	if (Members.HasMore())
	{
		check(Members.PeekKind() == EMemberKind::Range);
		if (Members.PeekName() == MemberIds[M::Interfaces])
		{
			LoadInterfacesStatic(Dst.Interfaces, Members.GrabRange().AsStructs());
		}
	}
	
	// Todo: Since VER_UE4_ADD_COOKED_TO_UCLASS the legacy system serializes a cooked bool,
	// this should already be known on the package level.
	Dst.bCooked = false;
}

UFunctionBinding::UFunctionBinding(TPropertySpecifier<NumMembers>& Spec)
: FunctionFlagsId(GetEnumId<Ids, EFunctionFlags>())
{
	Spec.Super = GUE.Structs.Struct;
	Spec.Members[M::FunctionFlags] = Specify<EFunctionFlags>(FunctionFlagsId);
	Spec.Members[M::EventGraphFunction] = GUE.Structs.ObjectPtr;
	Spec.Members[M::EventGraphCallOffset] = Specify<int32>();
}

void UFunctionBinding::Save(FMemberBuilder& Dst, const UFunction& Src, const UFunction*, const FSaveContext& Ctx) const
{
	SaveSuper(Dst, &Src, GUE.Structs.Struct, Ctx);
	Dst.AddEnum(MemberIds[M::FunctionFlags], FunctionFlagsId, Src.FunctionFlags);
	Dst.AddStruct(MemberIds[M::EventGraphFunction], GUE.Structs.ObjectPtr, SaveStruct(&Src.EventGraphFunction, GUE.Structs.ObjectPtr, Ctx));
	Dst.Add(MemberIds[M::EventGraphCallOffset], Src.EventGraphCallOffset);
}

void UFunctionBinding::Load(UFunction& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
{
	FMemberLoader Members(Src);
	TryLoadSuper(&Dst, Members);
	Dst.FunctionFlags = Members.GrabLeaf().As<EFunctionFlags>();
	LoadStruct(&Dst.EventGraphFunction, Members.GrabStruct());
	Dst.EventGraphCallOffset = Members.GrabLeaf().As<int32>();
	// Link is called from the derived meta class.
	check(!Members.HasMore());
}

UScriptStructBinding::UScriptStructBinding(TPropertySpecifier<NumMembers>& Spec)
: StructFlagsId(GetEnumId<Ids, EStructFlags>())
{
	Spec.Super = GUE.Structs.Struct;
	Spec.Members[M::StructFlags] = Specify<EStructFlags>(StructFlagsId);
}

void UScriptStructBinding::Save(FMemberBuilder& Dst, const UScriptStruct& Src, const UScriptStruct*, const FSaveContext& Ctx) const
{
	SaveSuper(Dst, &Src, GUE.Structs.Struct, Ctx);
	EStructFlags SavedStructFlags = EStructFlags(Src.StructFlags & ~STRUCT_ComputedFlags);
	Dst.AddEnum(MemberIds[M::StructFlags], StructFlagsId, SavedStructFlags);
}

void UScriptStructBinding::Load(UScriptStruct& Dst, FStructLoadView Src, ECustomLoadMethod Method) const
{
	FMemberLoader Members(Src);
	TryLoadSuper(&Dst, Members);
	EStructFlags LoadedStructFlags = Members.GrabLeaf().As<EStructFlags>();
	Dst.StructFlags = EStructFlags((LoadedStructFlags & ~STRUCT_ComputedFlags) | (Dst.StructFlags & STRUCT_ComputedFlags));
	// Link is called from the derived meta class.
	check(!Members.HasMore());
}

////////////////////////////////////////////////////////////////////////////////////////////////

static bool IsNativeOrTransientStruct(const UStruct* Struct)
{
	check(!Struct->HasAnyFlags(RF_MarkAsNative));
	if (Struct->HasAnyCastFlags(CASTCLASS_UClass))
	{
		EClassFlags Flags = static_cast<const UClass*>(Struct)->ClassFlags;
		if (Flags & (CLASS_Native))
		{
			return true;
		}
	}
	else if (Struct->HasAnyCastFlags(CASTCLASS_UScriptStruct))
	{
		EStructFlags Flags = static_cast<const UScriptStruct*>(Struct)->StructFlags;
		if (Flags & (STRUCT_NoExport | STRUCT_Native))
		{
			return true;
		}
	}
	else if (Struct->HasAnyCastFlags(CASTCLASS_UFunction))
	{
		EFunctionFlags Flags = static_cast<const UFunction*>(Struct)->FunctionFlags;
		if (Flags & (FUNC_Native))
		{
			return true;
		}
	}
	const UPackage* Package = Struct->GetPackage();
	if (Package->HasAnyPackageFlags(PKG_CompiledIn))
	{
		return true;
	}
	if (Package == GetTransientPackage())
	{
		return true;
	}
	return false;
}

struct FStructBindingMeta final : ICustomBinding
{
	explicit FStructBindingMeta(FBindId InSuper, TConstArrayView<FMemberId> InMemberIds, TConstArrayView<FBindId> InBindIds)
	: Super(InSuper)
	, MemberIds(InMemberIds)
	, BindIds(InBindIds)
	{
	}

	virtual void SaveCustom(FMemberBuilder& Dst, const void* Src, FBaseline Base, const FSaveContext& Ctx) override
	{
		const void* Default = Base.Get();
		if (!Default || DiffCustom(Src, Default, Ctx))
		{
			Save(Dst, *static_cast<const UStruct*>(Src), static_cast<const UStruct*>(Default), Ctx);
		}
	}

	virtual void LoadCustom(void* Dst, FStructLoadView Src, ECustomLoadMethod Method) const override
	{
		check(Method == ECustomLoadMethod::Assign);
		Load(*static_cast<UStruct*>(Dst), Src);
	}

	virtual bool DiffCustom(const void* A, const void* B, const FBindContext&) const override
	{
		return A != B;
	}

	void Save(FMemberBuilder& Dst, const UStruct& Src, const UStruct*, const FSaveContext& Ctx) const
	{
		SaveSuper(Dst, &Src, Super, Ctx);
		using namespace UClass_Private;
		if (Src.ChildProperties)
		{
			const FMemberId* MemberId = &MemberIds[0];
			const FBindId* BindId = &BindIds[0];
			for (const FField* Child = Src.ChildProperties; Child; Child = Child->Next)
			{
				Dst.AddStruct(*MemberId++, *BindId++, SaveStruct(Child, IndexPropertyMeta(Child), Ctx));
			}
		}
		if (const UClass* Class = Cast<UClass>(&Src))
		{
			// Todo: non-composable hack, save CDO for UClass after the named and type declared meta properties
			Dst.AddStruct(MemberIds.Last(), GUE.Structs.ObjectPtr, SaveStruct(&((*Class).*_ClassDefaultObject), GUE.Structs.ObjectPtr, Ctx));
		}
	}

	void Load(UStruct& Dst, FStructLoadView Src) const
	{
		using namespace UClass_Private;
		FMemberLoader Members(Src);
		// First load the custom base binding members which will construct the FProperties,
		// then load the named and typed meta properties into them
		TryLoadSuper(&Dst, Members);

		UClass* Class = Cast<UClass>(&Dst);
		const int32 NumChildren = MemberIds.Num() - !!Class;

		FField* Child = Dst.ChildProperties;
		for (int32 I = 0; I < NumChildren; ++I)
		{
			check(Child);
			check(Members.HasMore());
			LoadStruct(Child, Members.GrabStruct());
			Child = Child->Next;
		}
		// Explicitly call Link for all types immediately after loading all named and typed meta properties
		Dst.Link(*Dst.GetLinker(), true);
		if (Class)
		{
			// Todo: non-composable hack, load the CDO for UClass after the named and type declared meta properties
			// Todo: The UClass interface arrray will also need to be loaded at this point in time
			check(Members.HasMore());
			LoadStruct(&((*Class).*_ClassDefaultObject), Members.GrabStruct());
		}
		check(!Members.HasMore());
	}

	FBindId Super;
	TArray<FMemberId> MemberIds;
	TArray<FBindId> BindIds;
};

class FPropertyBinderMeta
{
public:
	FPropertyBinderMeta(FBindId Id)
	: Owner(Id)
	{
	}

	void DeclareAndBind(const UStruct* Struct)
	{
		FDualStructId Super = GetSuper(Struct);
		BindAllMembers(Struct);
		FStructDeclarationPtr Declaration = Declare(Super);
		// Todo: Ownership / memory leak
		ICustomBinding* Leak = new FStructBindingMeta(Super, Names, Members);
		GUE.Customs.BindStruct(Owner, *Leak, FStructDeclarationPtr(Declaration), {});
	}

	FDualStructId GetSuper(const UStruct* Struct)
	{
		if (Struct->HasAnyCastFlags(CASTCLASS_UFunction))
		{
			return GUE.Structs.Function;
		}
		else if (Struct->HasAnyCastFlags(CASTCLASS_UClass))
		{
			FType Type = IndexStruct(Struct->GetClass());
			return FDualStructId(GUE.Names.IndexDeclId(Type)); // e.g. UBlueprintGeneratedClass
		}
		else if (Struct->HasAnyCastFlags(CASTCLASS_UScriptStruct))
		{
			return GUE.Structs.ScriptStruct;
		}
		return GUE.Structs.Struct;
	}

	void BindAllMembers(const UStruct* Struct)
	{
		for (const FField* Child = Struct->ChildProperties; Child; Child = Child->Next)
		{
			BindMember(static_cast<const FProperty*>(Child));
		}
		if (const UClass* Class = Cast<UClass>(Struct))
		{
			// Todo: non-composable hack, declare CDO for UClass in the generic meta binding
			FDualStructId Id = GUE.Structs.ObjectPtr;
			Members.Emplace(Id);
			Specs.Emplace(Id);
			Names.Emplace(GUE.Names.NameMember("ClassDefaultObject"));
		}
	}

	void BindMember(const FProperty* Property)
	{
		FDualStructId Id = IndexPropertyMeta(Property);
		Members.Emplace(Id);
		Specs.Emplace(Id);
		Names.Emplace(GUE.Names.NameMember(Property->GetFName()));
	}

	FStructDeclarationPtr Declare(FDeclId Super) const
	{
		return PlainProps::Declare({LowerCast(Owner), Super, 0, EMemberPresence::AllowSparse, Names, {Specs}});
	}

private:
	const FBindId									Owner;

	TArray<FMemberId, TInlineAllocator<64>>			Names;
	TArray<FBindId, TInlineAllocator<64>>			Members;
	TArray<FMemberSpec, TInlineAllocator<64>>		Specs;
};

void BindStructMeta(FBindId Id, const UStruct* Struct)
{
	if (!IsNativeOrTransientStruct(Struct))
	{
		FPropertyBinderMeta Binder(Id);
		Binder.DeclareAndBind(Struct);
	}
}

} // namespace PlainProps::UE

////////////////////////////////////////////////////////////////////////////////////////////////

namespace PlainProps
{
// Temporary way to add a fake /Script/CoreUObject scope to non-UEnums, not strictly required
template<> inline FScopeId GetOuterScope<UE::FRuntimeIds, TTypename<EClassFlags>>()		{ return UE::GUE.Scopes.CoreUObject; }
template<> inline FScopeId GetOuterScope<UE::FRuntimeIds, TTypename<EFunctionFlags>>()	{ return UE::GUE.Scopes.CoreUObject; }
template<> inline FScopeId GetOuterScope<UE::FRuntimeIds, TTypename<EStructFlags>>()	{ return UE::GUE.Scopes.CoreUObject; }
template<> inline FScopeId GetOuterScope<UE::FRuntimeIds, TTypename<EPropertyFlags>>()	{ return UE::GUE.Scopes.CoreUObject; }

}
