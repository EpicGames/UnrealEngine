// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Misc/EnumClassFlags.h"
#include "UObject/NameTypes.h"

class UObject;
class UPackage;
struct FSavePackageArgs;

namespace PlainProps::UE
{

enum class EBindMode : uint8 { All, Source, Runtime };

enum class EBatchType : uint8 { Plain, Linker} ; // todo: Iostore, Zen

enum class ERoundtrip : uint8
{
	None 		= 0,
	PP 			= 1 << 0,
	TPS 		= 1 << 1,
	UPS 		= 1 << 2,
	TextMemory	= 1 << 3,
	TextStable	= 1 << 4,
	JSON		= 1 << 5,
};
ENUM_CLASS_FLAGS(ERoundtrip);

struct FLinkerDiffFilter
{
	// Structs for which to bypass the native identical function
	TSet<FName> BypassNativeIdenticalStructs;
	// Structs for which to ignore any diffs
	TSet<FName> IgnoreStructs;
	// Struct specific properties for which to ignore any diffs
	TSet<TPair<FName, FName>> IgnorePropertiesForStructs;
	// Base class properties for which to ignore any diffs (more expensive to check)
	TMap<FName, FName> IgnorePropertiesForBases;
	// Cast flags for properties for which to ignore any diffs
	uint64 IgnoreCastFlags = 0;
};

using FDropPackagesFunc = void (*)(TArray<UPackage*>&&, int32);

PLAINPROPSUOBJECT_API void SchemaBindAllTypes(EBindMode Mode, EBatchType BatchType);
PLAINPROPSUOBJECT_API int32 RoundtripViaPlainBatch(TConstArrayView<UObject*> Objects, ERoundtrip Options);
PLAINPROPSUOBJECT_API int32 RoundtripViaLinkerBatch(
	TConstArrayView<UObject*> Objects,
	ERoundtrip Options,
	const FLinkerDiffFilter& DiffFilter,
	FDropPackagesFunc DropPackagesFunc);

PLAINPROPSUOBJECT_API bool SaveTestPackage(UPackage* Package, const FString& Filename, const FSavePackageArgs& SaveArgs);
PLAINPROPSUOBJECT_API UPackage* LoadTestPackage(const FString& Filename);

}
