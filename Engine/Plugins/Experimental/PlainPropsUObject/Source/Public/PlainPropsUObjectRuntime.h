// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PlainPropsBindCtti.h"
#include "PlainPropsDeclare.h"
#include "PlainPropsIndex.h"
#include "PlainPropsTypes.h"
#include "PlainPropsUeCoreBindings.h"
#include "Containers/Map.h"
#include "UObject/DynamicallyTypedValue.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ScriptDelegateFwd.h"

struct FFieldPath;
struct FInstancedStruct;
struct FReferencePropertyValue;
struct FVerseFunction;

namespace PlainProps::UE
{

struct FDefaultInstance { uint64 Ptr; };

class FDefaultStructs final : public IDefaultStructs
{
public:
	~FDefaultStructs();
	void									Bind(FBindId Id, const UScriptStruct* Struct);
	void									BindZeroes(FBindId Id, SIZE_T Size, uint32 Alignment);
	void									BindStatic(FBindId Id, const void* Struct);
	void									Drop(FBindId Id);
	virtual const void*						Get(FBindId Id) override;

	template<class T>
	void									BindZeroes(FBindId Id) { BindZeroes(Id, sizeof(T), alignof(T)); }

private:
	FMutableMemoryView						Zeroes;
	TBitArray<>								Instanced;
	TMap<FBindId, FDefaultInstance>			Instances;
#if DO_CHECK
	TBitArray<>								Bound;
#endif

	void									ReserveFlags(uint32 Idx);
};

struct FCommonScopeIds
{
	explicit FCommonScopeIds(TIdIndexer<FSensitiveName>& Names);
	
	FScopeId					CoreUObject;
	FScopeId					Engine; // Move out?
};

struct FCommonTypenameIds
{
	explicit FCommonTypenameIds(TIdIndexer<FSensitiveName>& Names);
	
	FConcreteTypenameId			Optional;
	FConcreteTypenameId			Map;
	FConcreteTypenameId			Set;
	FConcreteTypenameId			Pair;
	FConcreteTypenameId			LeafArray;
	FConcreteTypenameId			TrivialArray;
	FConcreteTypenameId			NonTrivialArray;
	FConcreteTypenameId			StaticArray;
	FConcreteTypenameId			TrivialOptional;
	FConcreteTypenameId			IntrusiveOptional;
	FConcreteTypenameId			NonIntrusiveOptional;
	FConcreteTypenameId			String;
	FConcreteTypenameId			Utf8String;
	FConcreteTypenameId			AnsiString;
	FConcreteTypenameId			VerseString;
};

struct FCommonEnumIds
{
	explicit FCommonEnumIds(const FCommonScopeIds& Scopes, TIdIndexer<FSensitiveName>& Names);
	
	FEnumId						PixelFormat;
};

struct FCommonPropertyStructIds
{
	explicit FCommonPropertyStructIds(const FCommonScopeIds& Scopes, TIdIndexer<FSensitiveName>& Names);

	FDualStructId				Property;
	FDualStructId				EnumProperty;
	FDualStructId				BoolProperty;
	FDualStructId				ByteProperty;
	FDualStructId				StructProperty;
	FDualStructId				ClassProperty;
	FDualStructId				ObjectPropertyBase;
	FDualStructId				ArrayProperty;
	FDualStructId				SetProperty;
	FDualStructId				MapProperty;
	FDualStructId				OptionalProperty;
};

struct FCommonStructIds
{
	explicit FCommonStructIds(const FCommonScopeIds& Scopes, TIdIndexer<FSensitiveName>& Names);
	
	FDualStructId				Name;
	FDualStructId				Text;
	FDualStructId				Guid;
	FDualStructId				FieldPath;
	FDualStructId				Struct;
	FDualStructId				Class;
	FDualStructId				Function;
	FDualStructId				ScriptStruct;
	FDualStructId				SoftObjectPath;
	FDualStructId				ClassPtr;
	FDualStructId				ObjectPtr;
	FDualStructId				WeakObjectPtr;
	FDualStructId				LazyObjectPtr;
	FDualStructId				SoftObjectPtr;
	FDualStructId				ScriptInterface;
	FDualStructId				Delegate;
	FDeclId						MulticastDelegate;
	FBindId						MulticastInlineDelegate;
	FDualStructId				MulticastSparseDelegate;
	FDualStructId				VerseFunction;
	FDualStructId				DynamicallyTypedValue;
	FDualStructId				ReferencePropertyValue;
	FDualStructId				PropertyBag;
	FDualStructId				InstancedStruct;
};

struct FCommonMemberIds
{
	explicit FCommonMemberIds(TIdIndexer<FSensitiveName>& Names);
	
	FMemberId					Key;
	FMemberId					Value;
	FMemberId					Assign;
	FMemberId					Remove;
	FMemberId					Insert;
	FMemberId					Modify;
	FMemberId					Id;
	FMemberId					Object;
	FMemberId					Function;
	FMemberId					Invocations;
	FMemberId					Path;
	FMemberId					Owner;
};

// temporary non-intrusive mapping from UEnum->FEnumId and UStruct->FBindId,
// these ids will be owned directly by UEnum and UStruct in a future integration
struct FDynamicIds
{
	void			AddEnum(const UEnum* Enum, FEnumId Id)			{ Enums.Add(Enum, Id); }
	void			AddStruct(const UStruct* Struct, FBindId Id)
	{
		// Don't store the new id when adding a duplicated schema for a nested schema struct a in custom binding.
		// Todo: Can be cleaned up when interleaving custom binding members with schema members.
		if (!Structs.Contains(Struct))
		{
			Structs.Add(Struct, FDualStructId(Id));
		}
	}

	FEnumId			GetEnum(const UEnum* Enum) const				{ return Enums.FindChecked(Enum); }
	FDualStructId	GetStruct(const UStruct* Struct) const			{ return Structs.FindChecked(Struct); }

private:
	TMap<const UEnum*, FEnumId> Enums;
	TMap<const UStruct*, FDualStructId> Structs;
};

struct FMemberMetadata
{
	bool IsValid() const { return !Flags.IsEmpty(); }
	void Reset() { Flags.Empty(); }

	TArray<EPropertyFlags> Flags;
};

struct FMetadataBindings
{
	explicit FMetadataBindings(FDebugIds In) : Debug(In) {}

	void 							BindMetadata(FBindId Id, TConstArrayView<EPropertyFlags> Members);
	void							DropMetadata(FBindId Id);

	TConstArrayView<EPropertyFlags> GetMemberFlags(FBindId Id) const;

private:
	TArray<FMemberMetadata> Metadatas;
	FDebugIds				Debug;
};

struct FGlobals
{
	FGlobals();

	TIdIndexer<FSensitiveName>	Names;
	FEnumDeclarations			Enums;
	FSchemaBindings				Schemas;
	FMetadataBindings 			Metadatas;
	FCustomBindingsBottom		Customs;
	FDefaultStructs				Defaults;
	FCommonScopeIds				Scopes;
	FCommonEnumIds				EnumIds;
	FCommonStructIds			Structs;
	FCommonPropertyStructIds	Properties;
	FCommonTypenameIds			Typenames;
	FCommonMemberIds			Members;
	FDynamicIds					DynamicIds;
	FNumeralGenerator			Numerals;
	//Upgrade::FHistory			Upgrades;
	FDebugIds					Debug;
};
PLAINPROPSUOBJECT_API extern FGlobals GUE;

struct FRuntimeIds
{
	static FNameId				IndexName(FAnsiStringView Name)						{ return GUE.Names.MakeName(FName(Name)); }
	static FMemberId			IndexMember(FAnsiStringView Name)					{ return GUE.Names.NameMember(FName(Name)); }
	static FConcreteTypenameId	IndexTypename(FAnsiStringView Name)					{ return GUE.Names.NameType(FName(Name)); }
	static FFlatScopeId			IndexScope(FAnsiStringView Name)					{ return GUE.Names.NameScope(FName(Name)); }
	static FEnumId				IndexEnum(FScopeId Scope, FAnsiStringView Name)		{ return GUE.Names.IndexEnum({Scope, GUE.Names.MakeTypename(Name)}); }
	static FStructId			IndexStruct(FScopeId Scope, FAnsiStringView Name)	{ return GUE.Names.IndexStruct({Scope, GUE.Names.MakeTypename(Name)}); }
	static FEnumId				IndexEnum(FType Type)								{ return GUE.Names.IndexEnum(Type); }
	static FStructId			IndexStruct(FType Type)								{ return GUE.Names.IndexStruct(Type); }


	static FIdIndexerBase&		GetIndexer()							{ return GUE.Names; }
};

struct FDefaultRuntime
{
	using Ids = FRuntimeIds;
	template<class T> using CustomBindings = TCustomBind<T>;

	static FEnumDeclarations&	GetEnums()			{ return GUE.Enums; }
	static FSchemaBindings&		GetSchemas()		{ return GUE.Schemas; }
	static FCustomBindings&		GetCustoms()		{ return GUE.Customs; }
	static IDefaultStructs*		GetDefaults()		{ return nullptr; }
};

struct FDeltaRuntime : FDefaultRuntime
{
	template<class T> using CustomBindings = TCustomDeltaBind<T>;

	static IDefaultStructs*		GetDefaults()		{ return &GUE.Defaults; }
};

//////////////////////////////////////////////////////////////////////////

static constexpr ESizeType DefaultRangeMax = RangeSizeOf(FDefaultAllocator::SizeType{});

inline FMemberSpec DefaultRangeOf(FMemberSpec Item) { return FMemberSpec(DefaultRangeMax, Item); }

template<uint32 N>
using TPropertySpecifier = TCustomSpecifier<FRuntimeIds, N>;

// Default runtime scope to declare a ctti reflected native enum
template<typename Enum, EEnumMode Mode>
using TScopedDefaultEnumDeclaration = TScopedEnumDeclaration<Enum, Mode, FDefaultRuntime>;

// Runtime scope to declare and custom bind an override for a schema bound UStruct
template<class T, class Runtime, typename CustomBinding>
struct TScopedUStructBinding : TScopedStructBinding<T, Runtime, CustomBinding>
{
	TScopedUStructBinding()
	: TScopedStructBinding<T, Runtime, CustomBinding>(GUE.DynamicIds.GetStruct(T::StaticStruct()))
	{}
};

// Runtime scope to declare and custom bind an override for a schema bound UClass
template<class T, class Runtime, typename CustomBinding>
struct TScopedUClassBinding : TScopedStructBinding<T, Runtime, CustomBinding>
{
	TScopedUClassBinding()
	: TScopedStructBinding<T, Runtime, CustomBinding>(GUE.DynamicIds.GetStruct(T::StaticClass()))
	{}
};

// Default runtime scope to declare and custom bind an override for a schema bound UStruct
template<class T, typename CustomBinding>
using TScopedDefaultUStructBinding = TScopedUStructBinding<T, FDefaultRuntime, CustomBinding>;

// Default runtime scope to declare and custom bind an override for a schema bound UClass
template<class T, typename CustomBinding>
using TScopedDefaultUClassBinding = TScopedUClassBinding<T, FDefaultRuntime, CustomBinding>;

//////////////////////////////////////////////////////////////////////////

template <Enumeration E>
struct TUEnumAsByteBinding : ICustomBinding
{
	using Type = TEnumAsByte<E>;

	const FMemberId				MemberIds[1];
	const FEnumId				EnumId;

	TUEnumAsByteBinding(TPropertySpecifier<1>& Spec)
	: MemberIds{GUE.Members.Value}
	, EnumId(GetEnumId<FRuntimeIds, E>())
	{
		static_assert(sizeof(Type) == 1);
		Spec.Members[0] = FMemberSpec(ELeafType::Enum, ELeafWidth::B8, EnumId);
	}

	void Save(FMemberBuilder& Dst, const TEnumAsByte<E>& Src, const TEnumAsByte<E>* Default, const FSaveContext& Ctx) const
	{
		Dst.AddEnum(MemberIds[0], EnumId, Src.GetIntValue());
	}
	static void Plan(FMemcpyLoadPlan& Out) { Out.Size = sizeof(Type); }
	static bool Diff(TEnumAsByte<E> A, TEnumAsByte<E> B, const FBindContext&) { return A != B; }
};

//////////////////////////////////////////////////////////////////////////
struct FFieldPathBinding : ICustomBinding
{
	using Type = FFieldPath;
	const FMemberId MemberIds[2];

	FFieldPathBinding(TPropertySpecifier<2>& Spec);

	void			Save(FMemberBuilder& Dst, const Type& Src, const Type* Default, const FSaveContext&) const;
	void			Load(Type& Dst, FStructLoadView Src, ECustomLoadMethod Method) const;
	static bool		Diff(const Type& A, const Type& B, const FBindContext&);
};

struct FDelegateBinding : ICustomBinding
{
	using Type = FScriptDelegate;
	const FMemberId MemberIds[2];

	FDelegateBinding(TPropertySpecifier<2>& Spec);
	
	void			Save(FMemberBuilder& Dst, const Type& Src, const Type* Default, const FSaveContext&) const;
	void			Load(Type& Dst, FStructLoadView Src, ECustomLoadMethod Method) const;
	static bool		Diff(const Type& A, const Type& B, const FBindContext&);
};

struct FMulticastInlineDelegateBinding : ICustomBinding
{
	using Type = FMulticastScriptDelegate;
	const FMemberId MemberIds[1];

	FMulticastInlineDelegateBinding(TPropertySpecifier<1>& Spec);
	
	void			Save(FMemberBuilder& Dst, const Type& Src, const Type* Default, const FSaveContext&) const;
	void			Load(Type& Dst, FStructLoadView Src, ECustomLoadMethod Method) const;
	static bool		Diff(const Type& A, const Type& B, const FBindContext&);
};

//////////////////////////////////////////////////////////////////////////

struct FVerseFunctionBinding : ICustomBinding
{
	using Type =  FVerseFunction;
	const FMemberId MemberIds[1];

	FVerseFunctionBinding(TPropertySpecifier<1>& Spec);
	
	void			Save(FMemberBuilder& Dst, const Type& Src, const Type* Default, const FSaveContext&) const;
	void			Load(Type& Dst, FStructLoadView Src, ECustomLoadMethod Method) const;
	static bool		Diff(const Type& A, const Type& B, const FBindContext&);
};

struct FDynamicallyTypedValueBinding : ICustomBinding
{
	using Type = ::UE::FDynamicallyTypedValue;
	const FMemberId MemberIds[1];

	FDynamicallyTypedValueBinding(TPropertySpecifier<1>& Spec);
	
	void			Save(FMemberBuilder& Dst, const Type& Src, const Type* Default, const FSaveContext&) const;
	void			Load(Type& Dst, FStructLoadView Src, ECustomLoadMethod Method) const;
	static bool		Diff(const Type& A, const Type& B, const FBindContext&);
};

struct FReferencePropertyBinding : ICustomBinding
{
	using Type = FReferencePropertyValue;
	const FMemberId MemberIds[1];

	FReferencePropertyBinding(TPropertySpecifier<1>& Spec);
	
	void			Save(FMemberBuilder& Dst, const Type& Src, const Type* Default, const FSaveContext&) const;
	void			Load(Type& Dst, FStructLoadView Src, ECustomLoadMethod Method) const;
	static bool		Diff(const Type& A, const Type& B, const FBindContext&);
};

struct FInstancedStructBinding : ICustomBinding
{
	using Type = FInstancedStruct;
	const FMemberId MemberIds[2];

	FInstancedStructBinding(TPropertySpecifier<2>& Spec);

	void			Save(FMemberBuilder& Dst, const Type& Src, const Type* Default, const FSaveContext&) const;
	void			Load(Type& Dst, FStructLoadView Src, ECustomLoadMethod Method) const;
	static bool		Diff(const Type& A, const Type& B, const FBindContext&);
};

} // namespace PlainProps::UE

namespace PlainProps
{
template<Enumeration E> requires (requires { StaticEnum<E>(); })
struct TCustomBind<TEnumAsByte<E>> { using Type = UE::TUEnumAsByteBinding<E>; };

template<> struct TExternallyBound<FGuid, UE::FDefaultRuntime> : std::true_type {};
template<> struct TExternallyBound<FColor, UE::FDefaultRuntime> : std::true_type {};
template<> struct TExternallyBound<FLinearColor, UE::FDefaultRuntime> : std::true_type {};
template<> struct TExternallyBound<FTransform, UE::FDefaultRuntime> : std::true_type {};

// Enable usage of TObjectPtr<T> as members in ctti reflected structs
template<typename T> struct	TTypename<TObjectPtr<T>>	{ static_assert(!sizeof(T), "Unsupported type for TObjectPtr"); };
template<> struct TTypename<TObjectPtr<UClass>>			{ inline static constexpr std::string_view DeclName = "ClassPtr"; };
template<> struct TTypename<TObjectPtr<UObject>>		{ inline static constexpr std::string_view DeclName = "ObjectPtr"; };
template<> inline FScopeId GetOuterScope<UE::FRuntimeIds, TTypename<TObjectPtr<UClass>>>()	{ return UE::GUE.Scopes.CoreUObject; }
template<> inline FScopeId GetOuterScope<UE::FRuntimeIds, TTypename<TObjectPtr<UObject>>>()	{ return UE::GUE.Scopes.CoreUObject; }

// custom bindings
template<> struct TCustomBind<FFieldPath> { using Type = UE::FFieldPathBinding; };
template<> struct TCustomBind<FScriptDelegate> { using Type = UE::FDelegateBinding; };
template<> struct TCustomBind<FMulticastScriptDelegate> { using Type = UE::FMulticastInlineDelegateBinding; };
template<> struct TCustomBind<::UE::FDynamicallyTypedValue> { using Type = UE::FDynamicallyTypedValueBinding; };
template<> struct TCustomBind<FReferencePropertyValue> { using Type = UE::FReferencePropertyBinding; };
template<> struct TCustomBind<FVerseFunction> { using Type = UE::FVerseFunctionBinding; };
template<> struct TCustomBind<FInstancedStruct> { using Type = UE::FInstancedStructBinding; };

// Temporary way to tie certain Core types to /Script/CoreUObject scope
template<> inline FScopeId GetOuterScope<UE::FRuntimeIds, TTypename<FName>>()			{ return UE::GUE.Scopes.CoreUObject; }
template<> inline FScopeId GetOuterScope<UE::FRuntimeIds, TTypename<FText>>()			{ return UE::GUE.Scopes.CoreUObject; }
template<> inline FScopeId GetOuterScope<UE::FRuntimeIds, TTypename<FIntPoint>>()		{ return UE::GUE.Scopes.CoreUObject; }
template<> inline FScopeId GetOuterScope<UE::FRuntimeIds, TTypename<FQuat>>()			{ return UE::GUE.Scopes.CoreUObject; }
template<> inline FScopeId GetOuterScope<UE::FRuntimeIds, TTypename<FTransform>>()		{ return UE::GUE.Scopes.CoreUObject; }
template<> inline FScopeId GetOuterScope<UE::FRuntimeIds, TTypename<FVector>>()			{ return UE::GUE.Scopes.CoreUObject; }
template<> inline FScopeId GetOuterScope<UE::FRuntimeIds, TTypename<FVector4>>()		{ return UE::GUE.Scopes.CoreUObject; }
template<> inline FScopeId GetOuterScope<UE::FRuntimeIds, TTypename<FGuid>>()			{ return UE::GUE.Scopes.CoreUObject; }
template<> inline FScopeId GetOuterScope<UE::FRuntimeIds, TTypename<FColor>>()			{ return UE::GUE.Scopes.CoreUObject; }
template<> inline FScopeId GetOuterScope<UE::FRuntimeIds, TTypename<FLinearColor>>()	{ return UE::GUE.Scopes.CoreUObject; }

}
